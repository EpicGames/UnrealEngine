// Copyright Epic Games, Inc. All Rights Reserved.

#include "CommandLine/CmdLineParameters.h"
#include "Logging/SubmitToolLog.h"
#include "HAL/FileManager.h"
#include "Misc/CommandLine.h"
#include "Misc/FileHelper.h"
#include "String/ParseTokens.h"

const FCmdLineParameters FCmdLineParameters::Instance = FCmdLineParameters();

FCmdLineParameters::FCmdLineParameters()
	: Parameters(FSubmitToolCmdLine::SubmitToolCmdLineArgs)
{
}

const TCHAR* FCmdLineParameters::InitializeParameters() const
{
	UE_LOGF(LogSubmitTool, Log, "Original command line: %ls", FCommandLine::Get());
	ConfigurePerforce();
	ConfigureCoreLimit();
	ConfigureTrace();
	UE_LOGF(LogSubmitTool, Log, "Modified command line: %ls", FCommandLine::Get());
	return FCommandLine::Get();
}

void FCmdLineParameters::ConfigureCoreLimit() const
{
	// Setup corelimit if an existing option is not found on the command line
	FString CommandLine = FCommandLine::Get();
	if (CommandLine.Find("-corelimit") == INDEX_NONE)
	{
		FCommandLine::Append(TEXT(" -corelimit=16"));
	}
}

void FCmdLineParameters::ConfigurePerforce() const
{
	const TCHAR* CommandLine = FCommandLine::Get();

	FString ServerAndPort;
	FString UserName;
	FString ClientName;

	// 1. Check if all perforce parameters have been supplied on the command line
	int32 NumParameters = 0;
	NumParameters += FParse::Value(CommandLine, *FSubmitToolCmdLine::P4Server, ServerAndPort);
	NumParameters += FParse::Value(CommandLine, *FSubmitToolCmdLine::P4User, UserName);
	NumParameters += FParse::Value(CommandLine, *FSubmitToolCmdLine::P4Client, ClientName);

	UE_LOGF(LogSubmitTool, Log, "Perforce parameters after reading command line: Server: '%ls' | User: '%ls' | Workspace: '%ls'", *ServerAndPort, *UserName, *ClientName);
	if (NumParameters == 3)
	{
		return;
	}

	// 2. Fill missing perforce parameters from an existing p4config file in the root directory.
	// The cl to review is expected to be in the same P4Client as the directory where SubmitTool is running.
	TArray<FString, TInlineAllocator<3>> ConfigFilenames;
	if (FString Filename = FPlatformMisc::GetEnvironmentVariable(TEXT("P4CONFIG")); !Filename.IsEmpty())
	{
		ConfigFilenames.Add(MoveTemp(Filename));
	}
	else
	{
		ConfigFilenames.Append({"p4config.txt", ".p4config.txt", ".p4config"});
	}
	for (const FString& P4ConfigFilename : ConfigFilenames)
	{
		FString P4ConfigFilePath = FPaths::RootDir() / P4ConfigFilename;
		if (IFileManager::Get().FileExists(*P4ConfigFilePath))
		{
			FFileHelper::LoadFileToStringWithLineVisitor(
				*P4ConfigFilePath,
				[&NumParameters, &ServerAndPort, &UserName, &ClientName](FStringView Line)
				{
					using namespace UE::String;
					TArray<FStringView, TInlineAllocator<2>> Tokens;
					ParseTokens(Line, '=', Tokens, EParseTokensOptions::Trim | EParseTokensOptions::SkipEmpty);
					if (Tokens.Num() != 2)
					{
						return;
					}
					if (Tokens[0] == TEXTVIEW("P4PORT"))
					{
						if (ServerAndPort.IsEmpty())
						{
							++NumParameters;
							ServerAndPort = Tokens[1];
							FCommandLine::Append(*WriteToString<128>(" -", FSubmitToolCmdLine::P4Server, " ", ServerAndPort));
						}
					}
					else if (Tokens[0] == TEXTVIEW("P4USER"))
					{
						if (UserName.IsEmpty())
						{
							++NumParameters;
							UserName = Tokens[1];
							FCommandLine::Append(*WriteToString<128>(" -", FSubmitToolCmdLine::P4User, " ", UserName));
						}
					}
					else if (Tokens[0] == TEXTVIEW("P4CLIENT"))
					{
						if (ClientName.IsEmpty())
						{
							++NumParameters;
							ClientName = Tokens[1];
							FCommandLine::Append(*WriteToString<128>(" -", FSubmitToolCmdLine::P4Client, " ", ClientName));
						}
					}
				});
			UE_LOGF(LogSubmitTool, Log, "Perforce parameters after reading file \"%ls\": Server: '%ls' | User: '%ls' | Workspace: '%ls'",
				*P4ConfigFilePath, *ServerAndPort, *UserName, *ClientName);
			if (NumParameters == 3)
			{
				return;
			}
		}
	}

	// 3. Fill missing perforce parameters from P4 environment variables.
	if (ServerAndPort.IsEmpty())
	{
		ServerAndPort = FPlatformMisc::GetEnvironmentVariable(TEXT("P4PORT"));
		FCommandLine::Append(*WriteToString<128>(" -", FSubmitToolCmdLine::P4Server, " ", ServerAndPort));
	}
	if (UserName.IsEmpty())
	{
		UserName = FPlatformMisc::GetEnvironmentVariable(TEXT("P4USER"));
		FCommandLine::Append(*WriteToString<128>(" -", FSubmitToolCmdLine::P4User, " ", UserName));
	}
	if (ClientName.IsEmpty())
	{
		ClientName = FPlatformMisc::GetEnvironmentVariable(TEXT("P4CLIENT"));;
		FCommandLine::Append(*WriteToString<128>(" -", FSubmitToolCmdLine::P4Client, " ", ClientName));
	}
	UE_LOGF(LogSubmitTool, Log, "Perforce parameters after reading environment variables: Server: '%ls' | User: '%ls' | Workspace: '%ls'",
		*ServerAndPort, *UserName, *ClientName);
}


void FCmdLineParameters::ConfigureTrace() const
{
	// Setup tracing to file if no existing trace options are found on the command line
	FString CommandLine = FCommandLine::Get();
	if (CommandLine.Find("-trace") == INDEX_NONE)
	{
		FString TraceFile = FPaths::Combine(FPaths::ProjectLogDir(), "SubmitTool.utrace");
		TraceFile = FPaths::ConvertRelativePathToFull(TraceFile);
		if (IFileManager::Get().FileExists(*TraceFile))
		{
			IFileManager::Get().Delete(*TraceFile);
		}
		FCommandLine::Append(*WriteToString<128>(" -trace=cpu,log,bookmark,region,tasks -tracefile=\"", *TraceFile, "\""));
	}
}

bool FCmdLineParameters::ValidateParameters() const
{
	const TCHAR* CommandLine = FCommandLine::Get();

	bool bIsValid = true;
	for (const TSharedPtr<FCmdLineParameter>& Parameter : Parameters)
	{
		if (Parameter->bIsRequired)
		{
			FString Key = Parameter->Key;
			if(!Key.EndsWith(TEXT(" ")))
			{
				Key += TEXT(" ");
			}

			FString Val;
			if (FParse::Value(CommandLine, *Key, Val))
			{
				if (Val.IsEmpty())
				{
					UE_LOGF(LogSubmitTool, Error, "Command Line argument '-%ls' has no value.", *Parameter->Key);
					bIsValid = false;
				}
				else if(!Parameter->IsValid(Val))
				{
					UE_LOGF(LogSubmitTool, Error, "Command Line argument '-%ls' value '%ls' is invalid.", *Parameter->Key, *Val);
					bIsValid = false;
				}
			}
			else
			{
				UE_LOGF(LogSubmitTool, Error, "Command Line missing '-%ls' argument.", *Parameter->Key);
				bIsValid = false;
			}
		}
		else
		{
			// optional parameters must contain a value if they are present in the command line
			FString Val;
			if (!Parameter->bIsFlag && FParse::Value(CommandLine, *Parameter->Key, Val))
			{
				if (Val.IsEmpty())
				{
					UE_LOGF(LogSubmitTool, Error, "Command Line argument '-%ls' has no value.", *Parameter->Key);
					bIsValid = false;
				}
			}
		}
	}

	return bIsValid;
}

void FCmdLineParameters::LogParameters() const
{
	for (const TSharedPtr<FCmdLineParameter>& Parameter : Parameters)
	{
		UE_LOGF(LogSubmitTool, Warning, "-%ls\t%ls", *Parameter->Key, *Parameter->Description);
	}
}

const FCmdLineParameters& FCmdLineParameters::Get()
{
	return Instance;
}

bool FCmdLineParameters::Contains(const FString& InKey) const
{
	return FParse::Param(FCommandLine::Get(), *InKey);
}

bool FCmdLineParameters::GetValue(const FString& InKey, FString& OutValue) const
{
	const TCHAR* CommandLine = FCommandLine::Get();

	TArray<FString> Tokens;
	TArray<FString> Switches;
	FCommandLine::Parse(CommandLine, Tokens, Switches);
	const TSharedPtr<FCmdLineParameter>* Definition = Parameters.FindByPredicate([InKey](const TSharedPtr<FCmdLineParameter>& InCmdParam) { return InCmdParam->Key == InKey; });

	for (int i = 0; i < Switches.Num(); i++)
	{
		FString Switch = Switches[i];

		if (Switch.StartsWith(InKey, ESearchCase::IgnoreCase))
		{
			TArray<FString> SplitSwitch;
			if (2 == Switch.ParseIntoArray(SplitSwitch, TEXT("="), true))
			{
				OutValue = SplitSwitch[1];
				if (Definition)
				{
					(*Definition)->CustomParse(OutValue);
				}

				return true;
			}
		}
	}

	for(int i = 0; i < Tokens.Num(); i++)
	{
		if(Tokens[i].Equals(InKey, ESearchCase::IgnoreCase) && i + 1 < Tokens.Num())
		{
			if (Definition != nullptr)
			{
				OutValue = (*Definition)->bIsFlag ? TEXT("true") : Tokens[i + 1];
				(*Definition)->CustomParse(OutValue);
				return true;
			}
		}
	}

	return false;
}


bool FCmdLineParameters::GetValueArray(const FString& InKey, TArray<FString>& OutValue, FString&& Delimiter) const
{
	FString StringValue;
	if (GetValue(InKey, StringValue))
	{
		StringValue.ParseIntoArray(OutValue, *Delimiter);
		return true;
	}

	return false;
}