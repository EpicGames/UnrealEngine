// Copyright Epic Games, Inc. All Rights Reserved.

#include "Configuration/Configuration.h"

#include "Logging/SubmitToolLog.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "SubmitToolCoreUtils.h"
#include "Serialization/JsonSerializer.h"
#include "CommandLine/CmdLineParameters.h"

TSharedPtr<FConfiguration> FConfiguration::Instance;

void FConfiguration::Init()
{
	FConfiguration Configuration;

	FString RootDir;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::RootDir, RootDir);
	if (RootDir.Len() == 0)
	{
		// fallback to one up from engine dir if nothing was passed in
		RootDir = FPaths::ConvertRelativePathToFull(FPaths::EngineDir() + TEXT(".."));
	}

	FPaths::NormalizeDirectoryName(RootDir);

	Configuration.Values.Add(TEXT("$(root)"), RootDir);

	const FString EngineDir = RootDir + TEXT("/Engine");
	Configuration.Values.Add(TEXT("$(engine)"), EngineDir);

	const FString BatchFileDir = EngineDir + TEXT("/Build/BatchFiles");

	Configuration.Values.Add(TEXT("$(PackagedRoot)"), FPaths::ConvertRelativePathToFull(FPaths::EngineDir() + TEXT("..")));
	UE_LOGF(LogSubmitToolDebug, Log, "Packaged Root %ls", *FPaths::ConvertRelativePathToFull(FPaths::EngineDir() + TEXT("..")));

#if PLATFORM_WINDOWS
	Configuration.Values.Add(TEXT("$(ScriptExt)"), TEXT(".bat"));
	Configuration.Values.Add(TEXT("$(RunUAT)"), BatchFileDir + TEXT("/RunUAT.bat"));
	Configuration.Values.Add(TEXT("$(RunUBT)"), BatchFileDir + TEXT("/RunUBT.bat"));
#elif (PLATFORM_LINUX || PLATFORM_MAC)
	Configuration.Values.Add(TEXT("$(ScriptExt)"), TEXT(".sh"));
	Configuration.Values.Add(TEXT("$(RunUAT)"), BatchFileDir + TEXT("/RunUAT.sh"));
	Configuration.Values.Add(TEXT("$(RunUBT)"), BatchFileDir + TEXT("/RunUBT.sh"));
#else
	UE_LOGF(LogSubmitTool, Error, "Unknown platform, cannot resolve aliases $(RunUAT) and $(RunUBT)");
#endif

	Configuration.Values.Add(TEXT("$(UBTPlatform)"), FGenericPlatformMisc::GetUBTPlatform());
	
	FString Changelist;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4ChangeList, Changelist);
	Configuration.Values.Add(TEXT("$(CL)"), Changelist);

	FString PerforceServerAndPort;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4Server, PerforceServerAndPort);
	Configuration.Values.Add(TEXT("$(SERVER)"), PerforceServerAndPort);

	FString PerforceUserName;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4User, PerforceUserName);
	Configuration.Values.Add(TEXT("$(USER)"), PerforceUserName);

	FString PerforceClientName;
	FCmdLineParameters::Get().GetValue(FSubmitToolCmdLine::P4Client, PerforceClientName);
	Configuration.Values.Add(TEXT("$(CLIENT)"), PerforceClientName);

	Configuration.Values.Add(TEXT("$(localappdata)"), *FSubmitToolCoreUtils::GetLocalAppDataPath());

	Configuration.Values.Add(TEXT("$(SubmitToolSavedDir)"), *FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir()));

	Configuration.Values.Add(TEXT("$(SubmitToolLogsDir)"), *FPaths::ConvertRelativePathToFull(FPaths::ProjectLogDir()));

	Configuration.Values.Add(TEXT("$(SubmitToolConfigDir)"), *FPaths::ConvertRelativePathToFull(FPaths::ProjectConfigDir()));

	Configuration.Values.Add(TEXT("$(UGSSyncedCL)"), GetLastSyncedCLFromUGS(RootDir));

	Instance = MakeShared<FConfiguration>(Configuration);
}

void FConfiguration::AddOrUpdateEntry(const FString& Key, const FString& NewValue)
{
	if (Instance.IsValid())
	{
		if(Instance->Values.Contains(Key))
		{
			UE_LOGF(LogSubmitToolDebug, Log, "Configuration succesfully updated entry with key %ls, to %ls", *Key, *NewValue);
			Instance->Values[Key] = NewValue;
		}
		else
		{
			UE_LOGF(LogSubmitToolDebug, Log, "Configuration succesfully added entry with key %ls, to %ls", *Key, *NewValue);
			Instance->Values.Add(Key, NewValue);
		}
	}
	else
	{
		UE_LOGF(LogSubmitTool, Log, "Configuration failed to update entry with key %ls", *Key);
	}
}

FString FConfiguration::GetLastSyncedCLFromUGS(const FString& InRootDir)
{
	auto ReadFileProperty = [](const FString& InFile, const FString& InProperty) {
		FString FileContents;
		if (FPaths::FileExists(InFile) && FFileHelper::LoadFileToString(FileContents, *InFile))
		{
			TSharedPtr<FJsonObject> RootJsonObject;
			TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(FileContents);
			FJsonSerializer::Deserialize(Reader, RootJsonObject);
			int64 PropertyValue;
			if (RootJsonObject.IsValid() && RootJsonObject->TryGetNumberField(InProperty, PropertyValue))
			{
				return PropertyValue;
			}
		}
		return -1LL;
	};

	int64 StateJsonCL = ReadFileProperty(InRootDir / TEXT(".ugs/state.json"), TEXT("LastSyncChangeNumber"));
	int64 BuildVersionCL = ReadFileProperty(InRootDir / TEXT("Engine/Build/Build.version"), TEXT("Changelist"));

	return LexToString(FMath::Max(StateJsonCL, BuildVersionCL));
}

FString FConfiguration::Substitute(const FString& InStr)
{
	if (!Instance.IsValid())
	{
		return InStr;
	}

	FString Output(InStr);

	bool bReplaced = true;
	while (bReplaced)
	{
		// make sure we do not infinite loop
		bReplaced = false;

		// check if any entry is replaced
		for (const TPair<FString, FString>& Pair : Instance->Values)
		{
			if (Output.ReplaceInline(*Pair.Key, *Pair.Value, ESearchCase::IgnoreCase) != 0)
			{
				bReplaced = true;
			}
		}
	}

	return Output;
}


FString FConfiguration::SubstituteAndNormalizeFilename(const FString& InStr)
{
	FString Output = Substitute(InStr);

	FPaths::NormalizeFilename(Output);

	return Output;
}

FString FConfiguration::SubstituteAndNormalizeDirectory(const FString& InStr)
{
	FString Output = Substitute(InStr);

	FPaths::NormalizeDirectoryName(Output);

	return Output;
}
