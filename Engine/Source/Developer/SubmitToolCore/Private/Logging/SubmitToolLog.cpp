// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logging/SubmitToolLog.h"

DEFINE_LOG_CATEGORY(LogValidators);
DEFINE_LOG_CATEGORY(LogValidatorsResult);
DEFINE_LOG_CATEGORY(LogPresubmit);
DEFINE_LOG_CATEGORY(LogPresubmitResult);
DEFINE_LOG_CATEGORY(LogSubmitTool);
DEFINE_LOG_CATEGORY(LogSubmitToolDebug);
DEFINE_LOG_CATEGORY(LogSubmitToolP4);
DEFINE_LOG_CATEGORY(LogSubmitToolP4Debug);

