// Copyright Epic Games, Inc. All Rights Reserved.

#include "Logic/Validators/ValidatorFactory.h"

FValidatorFactory& FValidatorFactory::Get()
{
	static FValidatorFactory Instance;
	return Instance;
}

void FValidatorFactory::Register(const FString& TypeName, FValidatorConstructor InCreator)
{
	FString LowerCase = TypeName.ToLower();
	ensureAlwaysMsgf(!Registry.Contains(LowerCase),
		TEXT("Validator type '%s' was registered twice"), *TypeName);
	Registry.Add(LowerCase, MoveTemp(InCreator));
}

TSharedPtr<FValidatorBase> FValidatorFactory::Create(
	const FString& TypeName, const FName& Id,
	const FSubmitToolParameters& Params,
	const TSharedRef<FSubmitToolServiceProvider>& ServiceProvider,
	const FString& Definition) const
{
	const FValidatorConstructor* Creator = Registry.Find(TypeName.ToLower());
	if (!Creator)
	{
		return nullptr;
	}
	return (*Creator)(Id, Params, ServiceProvider, Definition);
}
