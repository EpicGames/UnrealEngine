// Copyright Epic Games, Inc. All Rights Reserved.


#include "Models/SubmitToolUserPrefs.h"
#include "Misc/Paths.h"
#include "Misc/StringOutputDevice.h"
#include "HAL/FileManager.h"

FString FSubmitToolUserPrefs::FilePath = FString();
FSubmitToolUserPrefs* FSubmitToolUserPrefs::Instance = nullptr;

FSubmitToolUserPrefs::~FSubmitToolUserPrefs()
{
	if(Instance != nullptr)
	{
		FString OutputText;
		FSubmitToolUserPrefs::StaticStruct()->ExportText(OutputText, Instance, Instance, nullptr, PPF_None, nullptr);

		FArchive* File = IFileManager::Get().CreateFileWriter(*FilePath, EFileWrite::FILEWRITE_EvenIfReadOnly);
		if (File != nullptr)
		{
			*File << OutputText;
			File->Close();
			delete File;
			File = nullptr;
			UE_LOGF(LogSubmitToolDebug, Verbose, "Saved User Prefs to %ls:\n%ls", *FilePath, *OutputText);
		}
		else
		{
			UE_LOGF(LogSubmitToolDebug, Verbose, "Failed to create a file for User Prefs with path %ls:", *FilePath);
		}
		Instance = nullptr;
	}
}

TUniquePtr<FSubmitToolUserPrefs> FSubmitToolUserPrefs::Initialize(const FString& InFilePath)
{
	FilePath = InFilePath;
	TUniquePtr<FSubmitToolUserPrefs> NewPrefs = MakeUnique<FSubmitToolUserPrefs>();
	if(FPaths::FileExists(FilePath))
	{
		FString InText;
		FArchive* File = IFileManager::Get().CreateFileReader(*FilePath, EFileRead::FILEREAD_None);
		if (File != nullptr)
		{
			*File << InText;
			File->Close();
			delete File;
			File = nullptr;
			FStringOutputDevice Errors;
			FSubmitToolUserPrefs::StaticStruct()->ImportText(*InText, NewPrefs.Get(), nullptr, 0, &Errors, FSubmitToolUserPrefs::StaticStruct()->GetName());

			if(!Errors.IsEmpty())
			{
				UE_LOGF(LogSubmitTool, Error, "Error loading User prefs file %ls, using defaults", *Errors);
			}
			else
			{
				UE_LOGF(LogSubmitToolDebug, Verbose, "Loaded User Prefs from %ls:\n%ls", *FilePath, *InText);
			}
		}
		else
		{
			UE_LOGF(LogSubmitToolDebug, Verbose, "Failed to open file for reading User Prefs %ls:", *FilePath);
		}

		if(Instance != nullptr)
		{
			UE_LOGF(LogSubmitTool, Warning, "UserPrefs have been reloaded");
		}
	}
	else
	{
		UE_LOGF(LogSubmitTool, Warning, "File %ls does not exist, generating one.", *FilePath);
	}

	Instance = NewPrefs.Get();

	return NewPrefs;
}
