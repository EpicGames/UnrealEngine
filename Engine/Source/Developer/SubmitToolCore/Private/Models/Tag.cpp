// Copyright Epic Games, Inc. All Rights Reserved.


#include "Models/Tag.h"
#include "Misc/StringBuilder.h"
#include "Logging/SubmitToolLog.h"
#include "Internationalization/Regex.h"


FTag::FTag(const FTagDefinition& def, size_t start) : Definition(def), StartPos(start)
{
	for(const TCHAR& c : Definition.ValueDelimiter)
	{
		int idx = SplittedDelims.Add(FString::Chr(c));
		Delimiters.Add(*SplittedDelims[idx]);
	}
}

FString FTag::GetFullTag() const
{
	TStringBuilder<256> strBuilder;

	strBuilder.AppendChar('\n');
	strBuilder.Append(TagKeyValue.IsEmpty() ? Definition.TagId : TagKeyValue);

	if(TagValues.Num() > 0)
	{
		strBuilder.AppendChar(' ');

		for(size_t i = 0; i < TagValues.Num(); ++i)
		{
			if(i != 0)
			{
				strBuilder.Append(Definition.ValueDelimiter);
			}
			strBuilder.Append(TagValues[i]);
		}
	}

	return strBuilder.ToString();
}

bool FTag::ParseTag(const FString& source)
{
	// regex pattern example, replacing tag, delimiter and min/maxvalues
	// (?:(?:\r\n|\r|\n)?TAGID(?= |\n|$))( +(?:[DELIMITERS]*(?!#)(?:[\w!"\$-\/\:-\@\[-\`\{-\~]+)){MINVALUES,MAXVALUES})?
	// (?:(?:\r\n|\r|\n)?#jira(?= |\n|$))( +(?:[, ]*(?!#)(?:[\w!"\$-\/\:-\@\[-\`\{-\~]+)){1,256})?

	const FString TagIdParse = Definition.RegexParseOverride.IsEmpty() ? Definition.GetTagId() : Definition.RegexParseOverride;
	const FString RegexPat = "(?:(?:\\r\\n|\\r|\\n)?(" + TagIdParse + ")(?= |\n|$))( +(?:[" + Definition.ValueDelimiter + "]*(?!#)(?:[\\w!\"\\$-\\/\\:-\\@\\[-\\`\\{-\\~]+)){" + FString::FromInt(Definition.MinValues) + "," + FString::FromInt(Definition.MaxValues) + "})?";
	FRegexPattern Pattern = FRegexPattern(RegexPat, ERegexPatternFlags::CaseInsensitive);
	FRegexMatcher Regex = FRegexMatcher(Pattern, source);
	bool Match = Regex.FindNext();
	if(Match)
	{
		bIsDirty = false;
		StartPos = Regex.GetMatchBeginning();
		LastSize = Regex.GetMatchEnding() - Regex.GetMatchBeginning();

		UE_LOGF(LogSubmitToolDebug, Log, "Start: %d", Regex.GetMatchBeginning());
		UE_LOGF(LogSubmitToolDebug, Log, "Regex matched: %ls", *Regex.GetCaptureGroup(0));

		TagKeyValue = Regex.GetCaptureGroup(1).TrimStartAndEnd();
		const FString Capture = Regex.GetCaptureGroup(2).TrimStart();
		Capture.ParseIntoArray(TagValues, Delimiters.GetData(), SplittedDelims.Num());

		for(const FString& Value : TagValues)
		{
			UE_LOGF(LogSubmitToolDebug, Log, "Captured Value: %ls", *Value);
		}

		UE_LOGF(LogSubmitToolDebug, Log, "End: %d", Regex.GetMatchEnding());
	}
	else
	{
		Reset();
		UE_LOGF(LogSubmitToolDebug, Log, "Tag %ls not found in description", *Definition.GetTagId());
	}

	if(OnTagUpdated.IsBound())
	{
		OnTagUpdated.Broadcast(*this);
	}
	return Match;
}

void FTag::SetValues(const FString& valuesText)
{
	bIsDirty = true;
	TagValues.Empty();
	valuesText.ParseIntoArray(TagValues, Delimiters.GetData(), SplittedDelims.Num());
	ValidationState = ETagState::Unchecked;

	if(OnTagUpdated.IsBound())
	{
		OnTagUpdated.Broadcast(*this);
	}
}

FString FTag::GetValuesText() const
{
	TStringBuilder<256> strBuilder;
	for(size_t i = 0; i < TagValues.Num(); ++i)
	{
		if(i != 0)
		{
			strBuilder.Append(Definition.ValueDelimiter);
		}
		strBuilder.Append(TagValues[i]);
	}

	return strBuilder.ToString();
}

void FTag::SetValues(const TArray<FString>& InValues)
{
	bIsDirty = true;
	this->TagValues = InValues;

	for(size_t i = 0; i<TagValues.Num();++i)
	{
		bool removedChar;
		do
		{
			removedChar = false;
			for(const FString& Delim : SplittedDelims)
			{
				bool localRemoved;
				do
				{
					TagValues[i].TrimCharInline(Delim[0], &localRemoved);
					removedChar |= localRemoved;
				} while(localRemoved);
			}
		} while(removedChar);
	}

	if(OnTagUpdated.IsBound())
	{
		OnTagUpdated.Broadcast(*this);
	}
}

void FTag::SetValues(TArray<FString>&& InValues)
{
	bIsDirty = true;
	this->TagValues = MoveTemp(InValues);

	for(size_t i = 0; i<TagValues.Num();++i)
	{
		bool removedChar;
		do
		{
			removedChar = false;
			for(const FString& Delim : SplittedDelims)
			{
				bool localRemoved;
				do
				{
					TagValues[i].TrimCharInline(Delim[0], &localRemoved);
					removedChar |= localRemoved;
				} while(localRemoved);
			}
		} while(removedChar);
	}

	if(OnTagUpdated.IsBound())
	{
		OnTagUpdated.Broadcast(*this);
	}
}

const TArray<FString> FTag::GetValues(bool bEvenIfDisabled) const
{
	if(IsEnabled() || bEvenIfDisabled)
	{
		return TagValues;
	}
	else
	{
		return TArray<FString>();
	}
}

const FTagValidationConfig& FTag::GetCurrentValidationConfig(const TArray<FSCFileRef>& InFiles) const
{
	if(Definition.ValidationOverrides.Num() != 0)
	{
		for(const FTagValidationOverride& ValidationOverride : Definition.ValidationOverrides)
		{
			FRegexPattern RegexPattern = FRegexPattern(ValidationOverride.RegexPath, ERegexPatternFlags::CaseInsensitive);
			for(const FSCFileRef& File : InFiles)
			{
				FRegexMatcher Regex = FRegexMatcher(RegexPattern, File->GetDepotPath());
				if(Regex.FindNext())
				{
					return ValidationOverride.ConfigOverride;
				}
			}
		}
	}

	return Definition.Validation;
}
