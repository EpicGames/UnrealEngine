// Copyright Epic Games, Inc. All Rights Reserved.

#include "SubmitToolCoreUtils.h"

#include "Misc/Paths.h"
#include "Logging/SubmitToolLog.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManagerGeneric.h"

#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#define LOCTEXT_NAMESPACE "FSubmitToolCoreUtils"

TMap<FString, TMap<bool, TSet<FString>>> FSubmitToolCoreUtils::HierarchyWildcardsCache;

FString FSubmitToolCoreUtils::GetLocalAppDataPath()
{
#if PLATFORM_WINDOWS
	FString LocalAppData = FPlatformMisc::GetEnvironmentVariable(TEXT("LOCALAPPDATA"));
	FPaths::NormalizeDirectoryName(LocalAppData);
#elif PLATFORM_MAC
	const FString LocalAppData = FPaths::Combine(FPlatformMisc::GetEnvironmentVariable(TEXT("HOME")), TEXT("Library"), TEXT("Application Support"));
#elif PLATFORM_LINUX
	const FString LocalAppData = FPaths::Combine(FPlatformMisc::GetEnvironmentVariable(TEXT("HOME")), TEXT(".local"), TEXT("share"));
#else
	static_assert(false);
#endif

	return LocalAppData;
}

void FSubmitToolCoreUtils::CopyDiagnosticFilesToClipboard(TConstArrayView<FString> Files)
{
#if PLATFORM_WINDOWS
	if (OpenClipboard(GetActiveWindow()))
	{
		verify(EmptyClipboard());
		HGLOBAL GlobalMem;
		SIZE_T RequiredSize = sizeof(DROPFILES) + sizeof(TCHAR);
		for (const FString& File : Files)
		{
			RequiredSize += (File.Len() * sizeof(TCHAR)) + sizeof(TCHAR);
		}
		GlobalMem = GlobalAlloc(GMEM_MOVEABLE, RequiredSize);
		check(GlobalMem);

		uint8* Data = (uint8*)GlobalLock(GlobalMem);
		DROPFILES* Drop = (DROPFILES*)Data;
		if (Drop == NULL)
		{
			UE_LOGF(LogSubmitTool, Error, "GlobalLock Failed with error code: %i", (uint32)GetLastError());
			GlobalFree(GlobalMem);
			return;
		}

		Drop->pFiles = sizeof(DROPFILES);
		Drop->fWide = 1;

		TCHAR* Dest = (TCHAR*)(Data + sizeof(DROPFILES));
		TCHAR* End = (TCHAR*)(Data + RequiredSize);
		for (const FString& File : Files)
		{
			FCString::Strncpy(Dest, *File, End - Dest);
			Dest += (File.Len() + 1);
		}

		if (SetClipboardData(CF_HDROP, GlobalMem) == NULL)
		{
			UE_LOGF(LogSubmitTool, Warning, "SetClipboardData failed with error code %i", (uint32)GetLastError());
			GlobalFree(GlobalMem);
			return;
		}

		GlobalUnlock(GlobalMem);

		verify(CloseClipboard());
	}
	else
	{
		UE_LOGF(LogSubmitTool, Warning, "OpenClipboard failed with error code %i", (uint32)GetLastError());
	}
#endif
}

bool FSubmitToolCoreUtils::IsFileInHierarchy(const FString& InWildcard, const FString& InPath)
{
	HierarchyWildcardsCache.FindOrAdd(InWildcard);	
	for (const FString& PathWithFile : HierarchyWildcardsCache[InWildcard].FindOrAdd(true))
	{			
		if (FPaths::IsUnderDirectory(InPath, PathWithFile))
		{
			return true;
		}
	}
	
	FString CurrentDirectory = InPath;
	if (!FPaths::GetExtension(InPath).IsEmpty())
	{
		CurrentDirectory = FPaths::GetPath(InPath);
	}

	const TSet<FString>& NegativeDirectories = HierarchyWildcardsCache[InWildcard].FindOrAdd(false);
	TSet<FString> WalkedDirs;
	bool bFound = false;

	while (!CurrentDirectory.IsEmpty())
	{
		// If we have walked this part of the hierarchy before, we can stop going up since we've already done it from here
		if (NegativeDirectories.Contains(CurrentDirectory))
		{
			break;
		}

		WalkedDirs.Add(CurrentDirectory);

		if (IFileManager::Get().DirectoryExists(*CurrentDirectory))
		{
			TArray<FString> FilesInFolder;
			IFileManager::Get().FindFiles(FilesInFolder, *(CurrentDirectory / InWildcard), true, false);

			bFound = FilesInFolder.Num() != 0;
			if (bFound)
			{
				// Add only the directory with the file to the found cache since we will test the whole tree with IsUnderDirectory
				HierarchyWildcardsCache[InWildcard].FindOrAdd(true).Add(CurrentDirectory);
				UE_LOGF(LogSubmitToolDebug, Log, "File '%ls' found in directory '%ls' for wildcard '%ls'. Tested path: '%ls'", *FilesInFolder[0], *CurrentDirectory, *InWildcard, *InPath);
				break;
			}
		}

		CurrentDirectory = FPaths::GetPath(CurrentDirectory);
	}

	// If we didn't find matches, add the whole walked hierarchy to the not found cache since negative tests are done against the individual leafs
	if (!bFound)
	{
		HierarchyWildcardsCache[InWildcard].FindOrAdd(false).Append(WalkedDirs);
	}

	return bFound;
}

#undef LOCTEXT_NAMESPACE
