// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FCmdLineParameter
{
public:
	FCmdLineParameter() = delete;
	FCmdLineParameter(FString InKey, bool InIsRequired, FString InDescription, bool InIsFlag = false, TFunction<bool(const FString& InValue)> InValidator = nullptr, TFunction<void(FString& InValue)> InParseInPlace = nullptr) :
		Key(InKey),
		bIsRequired(InIsRequired),
		bIsFlag(InIsFlag),
		Description(InDescription),
		Validator(InValidator),
		Parser(InParseInPlace)
	{}

	FString Key;
	bool bIsRequired;
	bool bIsFlag;
	FString Description;

	bool IsValid(const FString& value) const
	{
		if(Validator != nullptr)
		{
			return Validator(value);
		}

		return true;
	}

	void CustomParse(FString& InOutValue) const
	{
		if(Parser != nullptr)
		{
			Parser(InOutValue);
		}
	}


private:
	TFunction<bool(const FString& InParam)> Validator;
	TFunction<void(FString& InParam)> Parser;
};