// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CmdLineParameter.h"
#include "Misc/Paths.h"

namespace FSubmitToolCmdLine
{
	const FString P4Server = TEXT("server");
	const FString P4Client = TEXT("client");
	const FString P4User = TEXT("user");
	const FString P4ChangeList = TEXT("cl");
	const FString RootDir = TEXT("root-dir");
	const FString EditorFlag = TEXT("from-editor");
	const FString EnableLocks = TEXT("enablelocks");
	const FString TestMode = TEXT("test-mode");
	const FString WaitForDebugger = TEXT("wait-for-debugger");

	const TArray<TSharedPtr<FCmdLineParameter>> SubmitToolCmdLineArgs =
	{
		MakeShared<FCmdLineParameter>(P4Server, true, TEXT("Perforce Server information, expected with format '<address>:<port>'")),
		MakeShared<FCmdLineParameter>(P4Client, true, TEXT("Perforce workspace name.")),
		MakeShared<FCmdLineParameter>(P4User, true, TEXT("Perforce user name.")),
		MakeShared<FCmdLineParameter>(P4ChangeList, true, TEXT("Perforce changelist number to submit."), false, [](const FString& InValue) { return InValue.IsNumeric() || InValue.Equals(TEXT("default")); }),
		MakeShared<FCmdLineParameter>(RootDir, false, TEXT("Root directory for the branch this change is part of"), false, nullptr,
		[](FString& OutValue)
		{
			TArray<TCHAR> CharactersToRemove{ '\\', '\"' , '\''};

			size_t Left = 0;
			size_t Right = OutValue.Len() - 1;
			bool bKeepLooping = true;

			while (bKeepLooping)
			{
				bKeepLooping = false;

				if (CharactersToRemove.Contains(OutValue[Left]))
				{
					Left++;
					bKeepLooping = true;
				}

				if (CharactersToRemove.Contains(OutValue[Right]))
				{
					Right--;
					bKeepLooping = true;
				}
			}
			OutValue.MidInline(Left, (Right - Left) + 1);

			#if PLATFORM_WINDOWS
			if (OutValue.Len() > 1 && OutValue[1] == ':')
			{
				OutValue[0] = FChar::ToUpper(OutValue[0]);
			}
			#endif
			
			FPaths::NormalizeDirectoryName(OutValue);
		}),
		MakeShared<FCmdLineParameter>(EditorFlag, false, TEXT("Flag indicating if submit tool is being invoked from unreal editor."), true),
		MakeShared<FCmdLineParameter>(EnableLocks, false, TEXT("Enables lockdown files previously defined by the config by their id")),
		MakeShared<FCmdLineParameter>(TestMode, false, TEXT("Submit tool launched to execute RPC tests."), true),
		MakeShared<FCmdLineParameter>(WaitForDebugger, false, TEXT("Submit tool waits for a debugger to attach."), true)
	};
}

class FCmdLineParameters
{
public:
	FCmdLineParameters();
	const TCHAR* InitializeParameters() const;
	bool ValidateParameters() const;
	void LogParameters() const;

	const static FCmdLineParameters& Get();

	bool Contains(const FString& InKey) const;
	bool GetValue(const FString& InKey, FString& OutValue) const;
	bool GetValueArray(const FString& InKey, TArray<FString>& OutValue, FString&& Delimiter = TEXT(",")) const;
private:
	const static FCmdLineParameters Instance;
	void ConfigureCoreLimit() const;
	void ConfigurePerforce() const;
	void ConfigureTrace() const;

	const TArray<TSharedPtr<FCmdLineParameter>> Parameters;
};
