// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

class FConfiguration
{
public:
	static void Init();
	static FString Substitute(const FString& InStr);
	static FString SubstituteAndNormalizeFilename(const FString& InStr);
	static FString SubstituteAndNormalizeDirectory(const FString& InStr);
	static void AddOrUpdateEntry(const FString& Key, const FString& NewValue);

private:
	static FString GetLastSyncedCLFromUGS(const FString& InRootDir);
	static TSharedPtr<FConfiguration> Instance;
	TMap<FString, FString> Values;
};