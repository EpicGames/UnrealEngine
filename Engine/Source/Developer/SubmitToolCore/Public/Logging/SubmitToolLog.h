// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Logging/LogMacros.h"

DECLARE_LOG_CATEGORY_EXTERN(LogValidators, Verbose, All);
DECLARE_LOG_CATEGORY_EXTERN(LogValidatorsResult, Verbose, All);
DECLARE_LOG_CATEGORY_EXTERN(LogPresubmit, Verbose, All);
DECLARE_LOG_CATEGORY_EXTERN(LogPresubmitResult, Verbose, All);

DECLARE_LOG_CATEGORY_EXTERN(LogSubmitTool, Verbose, All);
DECLARE_LOG_CATEGORY_EXTERN(LogSubmitToolDebug, Verbose, All);
DECLARE_LOG_CATEGORY_EXTERN(LogSubmitToolP4, Verbose, All);
DECLARE_LOG_CATEGORY_EXTERN(LogSubmitToolP4Debug, Verbose, All);

