// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Models/CacheData.h"
#include "Models/SCFile.h"

struct FGeneralParameters;

class ICacheDataService : public ISubmitToolService
{
public:
	virtual bool GetChangelistCacheData(const FString& InCLId, FChangelistCacheData& OutCLData) const = 0;

	virtual const FDateTime GetLastValidationDate(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const FString& InDepotPath) const = 0;
	virtual void UpdateLastValidationForFiles(const FString& InCLId, const FName& InValidatorId, const FString& InValidatorConfig, const TArray<FSCFileRef>& InFiles, const FDateTime& InNewTimestamp) = 0;

	virtual FString GetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId) const = 0;
	virtual void SetIntegrationFieldValue(const FString& InCLId, const FString& InIntegrationFieldId, const FString& InValue) = 0;

	virtual FDateTime GetLastKnownShelfUpToDate(const FString& InCLId) const = 0;
	virtual void SetLastShelfKnownUpToDate(const FString& InCLId, const FDateTime& InDateTime) = 0;

	virtual void SaveCacheToDisk() = 0;
};

Expose_TNameOf(ICacheDataService);
