// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Logic/Services/Interfaces/ISTSourceControlService.h"
#include "Logic/Validators/ValidatorDefinition.h"
#include "Models/SCFile.h"

DECLARE_MULTICAST_DELEGATE(FOnCLDescriptionUpdated)
DECLARE_DELEGATE_OneParam(FOnChangeListReadyDelegate, bool /*bValidState*/)
DECLARE_DELEGATE_OneParam(FOnChangelistRefreshDelegate, ETaskArea /*ChangedValue*/)

class IChangelistService : public ISubmitToolService
{
public:
	virtual const FString GetCLID() const = 0;
	virtual const FString& GetCLDescription() = 0;
	virtual bool SetCLDescription(const FString& NewDescription, bool bNotifyEvent = false) = 0;
	virtual const TArray<FSCFileRef>& GetFilesInCL() const = 0;
	virtual const TArray<FSCFileRef>& GetShelvedFilesInCL() const = 0;
	virtual bool HasShelvedFiles() const = 0;
	virtual bool HasP4OperationsRunning() const = 0;
	virtual bool IsShelfReady() const = 0;
	virtual bool IsShelfComplete() const = 0;
	virtual const TMap<FString, FString>& GetOpenFilesInChangelists() const = 0;

	virtual void Init() = 0;
	virtual void Submit(const FString& InDescriptionAddendum = TEXT(""), TFunction<void(const FSCCommand&)> OnComplete = nullptr) = 0;
	virtual void CreateCLFromDefaultCL() = 0;
	virtual void FetchChangelistDataAsync() = 0;
	virtual void RevertUnchangedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) = 0;
	virtual void UpdateP4CLDescription() = 0;
	virtual void UpdateP4CLDescriptionSynchronously() = 0;
	virtual void DeleteShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) = 0;
	virtual void ReplaceShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) = 0;
	virtual void CreateShelvedFiles(TFunction<void(const FSCCommand&)> OnComplete = nullptr) = 0;
	virtual void UpdateShelvedFiles(TArray<FString> InFiles, TFunction<void(const FSCCommand&)> OnComplete = nullptr) = 0;
	virtual void CheckShelfIsUpToDate(TFunction<void(bool, TArray<FString>)> OnComplete) = 0;
	virtual void EnsureShelfIsCurrent() = 0;
	virtual void CancelP4Operations() = 0;

	FOnCLDescriptionUpdated OnCLDescriptionUpdated;
};

Expose_TNameOf(IChangelistService);
