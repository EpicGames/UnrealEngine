// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Models/JiraToken.h"
#include "Tasks/Task.h"

class ICredentialsService : public ISubmitToolService
{
public:
	virtual bool HasJiraCredentials() const = 0;
	virtual const FString& GetEncodedLoginString() const = 0;
	virtual FString GetUsername() const = 0;
	virtual bool IsOIDCTokenEnabled() = 0;
	virtual bool IsTokenReady() const = 0;
	virtual const FString& GetToken() const = 0;
	virtual UE::Tasks::TTask<void> QueueWorkForToken(TFunction<void(const FString&)> InFunction) = 0;

	virtual void InvalidateCredentials() = 0;
	virtual void SetLogin(const FString& InUsername, const FString& InPassword) = 0;
	virtual void SetJiraToken(FJiraToken&& InJiraToken) = 0;
	virtual const TUniquePtr<FJiraToken>& GetJiraToken() const = 0;
};

Expose_TNameOf(ICredentialsService);
