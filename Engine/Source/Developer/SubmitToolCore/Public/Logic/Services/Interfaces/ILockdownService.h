// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Models/LockdownData.h"
#include "Models/SCFile.h"

class ILockdownService : public ISubmitToolService
{
public:
	virtual FSubmitToolLockdownData ArePathsInLockdown(const TArray<FSCFileRef>& InPaths, const TSet<FString>& InDynamicFiles = TSet<FString>()) = 0;
	virtual bool IsBlockingOperationRunning() const = 0;
};

Expose_TNameOf(ILockdownService);
