// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Parameters/SubmitToolParameters.h"
#include "Memory/SharedBuffer.h"
#include "Tasks/Task.h"
#include "Misc/Base64.h"

struct FAuthTicket
{
public:
	FAuthTicket() = default;
	FAuthTicket(const FString& InUsername, const FString& InPassword) : Username(InUsername), Password(InPassword) {}
	FAuthTicket(FStringView TicketString)
	{
		int32 ChopIndex;
		if(TicketString.FindChar(':', ChopIndex))
		{
			Username = TicketString.Left(ChopIndex);
			Password = TicketString.RightChop(ChopIndex + 1);
		}
	}
	virtual ~FAuthTicket() {}

	virtual FString ToString() const
	{
		return TEXT("Basic ") + FBase64::Encode(Username + TEXT(":") + Password);
	}

	bool IsValid() const
	{
		return !Username.IsEmpty() && !Password.IsEmpty();
	}
	FString Username;
private:
	FString Password;
};

struct FUserData
{
	FUserData(const FString& Username, const FString& Name, const FString& Email) : Name(Name), Username(Username), Email(Email)
	{}

	FString Name;
	FString Username;
	FString Email;

	bool Contains(const FString& SubString) const
	{
		return Name.Contains(SubString, ESearchCase::IgnoreCase) ||
			Username.Contains(SubString, ESearchCase::IgnoreCase) ||
			Email.Contains(SubString, ESearchCase::IgnoreCase);
	}

	FString GetDisplayText()
	{
		return Name + " - " + Username + " - " + Email;
	}
};

struct FSCResultInfo
{
	/** Append any messages from another FSCResultInfo, ensuring to keep any already accumulated info. */
	void Append(const FSCResultInfo& InResultInfo)
	{
		InfoMessages.Append(InResultInfo.InfoMessages);
		ErrorMessages.Append(InResultInfo.ErrorMessages);
	}

	bool HasErrors() const
	{
		return !ErrorMessages.IsEmpty();
	}

	/** Info and/or warning message storage */
	TArray<FText> InfoMessages;

	/** Potential error message storage */
	TArray<FText> ErrorMessages;
};


struct FSCCStream
{
public:
	FSCCStream(const FString& InName, const FString& InParent, const FString& InType)
		: Name(InName), Parent(InParent), Type(InType)
	{}
	const FString Name;
	const FString Parent;
	const FString Type;

	TArray<const FString> AdditionalImportPaths;
};

using FSCCRecord = TMap<FString, FString>;
using FSCCRecordSet = TArray<FSCCRecord>;

struct FSCCommand
{
	// in
	FString Command;
	TArray<FString> Args;
	bool bCollectData = false;
	bool bSilentErrors = false;
	FString InputData;
	// out
	FSCCRecordSet Values;
	FSCResultInfo Results;
	TMap<FString, FSharedBuffer> Data;
	bool bSuccess = false;
	bool bDroppedConnection = false;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnUsersGet, TArray<TSharedPtr<FUserData>>& /*Users*/)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGroupsGet, TArray<TSharedPtr<FString>>& /*Groups*/)
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUsersAndGroupsGet, TArray<TSharedPtr<FUserData>>& /*Users*/, TArray<TSharedPtr<FString>>& /*Groups*/)

using FOnSCCCommandComplete = TDelegate<void(const FSCCommand&)>;

class ISTSourceControlService : public ISubmitToolService
{
public:
	virtual void InitializeParameters(const FSubmitToolParameters& InParameters) = 0;
	virtual void GetUsers(const FOnUsersGet::FDelegate& Callback) = 0;
	virtual void GetGroups(const FOnGroupsGet::FDelegate& Callback) = 0;
	virtual void GetUsersAndGroups(const FOnUsersAndGroupsGet::FDelegate& Callback) = 0;

	virtual UE::Tasks::TTask<bool> DownloadFiles(TArray<FString>&& FilePaths, TMap<FString, FSharedBuffer>& OutFileBuffers, bool bSilentErrors = false) = 0;
	virtual const TArray<TSharedPtr<FUserData>>& GetRecentUsers() const = 0;
	virtual void AddRecentUser(TSharedPtr<FUserData>& User) = 0;
	virtual const TArray<TSharedPtr<FString>>& GetRecentGroups() const = 0;
	virtual void AddRecentGroup(TSharedPtr<FString>& Group) = 0;

	virtual TSharedPtr<FUserData> GetUserDataFromCache(const FString& Username) const = 0;

	virtual const TArray<FSCCStream*>& GetClientStreams() const = 0;
	virtual const FSCCStream* GetSCCStream(const FString& InStreamName) = 0;
	virtual const FString GetRootStreamName() = 0;
	virtual const FString GetCurrentStreamName() = 0;
	virtual int32 GetDepotStreamDepth(const FString& InDepotName) = 0;
	virtual const FAuthTicket& GetAuthTicket() = 0;
	virtual void RequestCancel() = 0;
	virtual bool IsBusy() const = 0;

	void RunCommand(FSCCommand&& InCommand, FOnSCCCommandComplete InCompleteCallback)
	{
		check(InCompleteCallback.IsBound());
		TSharedRef<FSCCommand> Command = MakeShared<FSCCommand>(MoveTemp(InCommand));

		UE::Tasks::FTask CommandTask = UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Command, InCompleteCallback]
		{
			Command->bSuccess = RunCommandInternal(*Command);
			UE::Tasks::Launch(UE_SOURCE_LOCATION, [InCompleteCallback, Command]
			{
				TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*FString::Printf(TEXT("RunP4Command_%s_Callback"), *Command->Command));
				InCompleteCallback.Execute(*Command);
			}, LowLevelTasks::ETaskPriority::Normal, UE::Tasks::EExtendedTaskPriority::GameThreadNormalPri);
		});
	}

	// Runs the command in a task thread and the provided callback in a dependent GT task.
	void RunCommand(FString&& InCommand, TArray<FString>&& InAdditionalArgs, FOnSCCCommandComplete InCompleteCallback, bool bSilentErrors = false)
	{
		check(InCompleteCallback.IsBound());
		RunCommand(FSCCommand(MoveTemp(InCommand), MoveTemp(InAdditionalArgs), false, bSilentErrors), InCompleteCallback);
	}

	// Runs the command in a task thread, calling GetResult() will wait until the task is completed and return the command data.
	UE::Tasks::TTask<TSharedRef<FSCCommand>> RunCommand(FString&& InCommand, TArray<FString>&& InAdditionalArgs, bool bSilentErrors = false)
	{
		TSharedRef<FSCCommand> Command = MakeShared<FSCCommand>(MoveTemp(InCommand), MoveTemp(InAdditionalArgs), false, bSilentErrors);
		return UE::Tasks::Launch(UE_SOURCE_LOCATION, [this, Command, bSilentErrors]
		{
			Command->bSuccess = RunCommandInternal(*Command);
			return Command;
		});
	}

	// Runs the command synchronously and returns the command data
	FSCCommand RunSyncCommand(FString&& InCommand, TArray<FString>&& InAdditionalArgs, bool bSilentErrors = false)
	{
		FSCCommand Command(MoveTemp(InCommand), MoveTemp(InAdditionalArgs), false, bSilentErrors);
		Command.bSuccess = RunCommandInternal(Command);
		return MoveTemp(Command);
	}

	void RunSyncCommand(FSCCommand& InCommand)
	{
		InCommand.bSuccess = RunCommandInternal(InCommand);
	}
protected:
	virtual bool RunCommandInternal(FSCCommand& InCommand) = 0;

};

Expose_TNameOf(ISTSourceControlService);
