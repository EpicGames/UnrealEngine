// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Models/PreflightData.h"
#include "Models/SCFile.h"
#include "Parameters/SubmitToolParameters.h"

class FSubmitToolServiceProvider;

class ISubmitToolExtensions : public ISubmitToolService
{
public:
	virtual TOptional<FPreflightTemplateDefinition> SelectPreflightTemplate(
		const TArray<FSCFileRef>& InFilesInCL,
		const TSharedPtr<FSubmitToolServiceProvider>& InServiceProvider,
		const FHordeParameters& InHordeParams) const = 0;
};

Expose_TNameOf(ISubmitToolExtensions);
