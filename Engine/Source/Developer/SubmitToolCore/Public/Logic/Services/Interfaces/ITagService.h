// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"
#include "Models/Tag.h"

class ITagService : public ISubmitToolService
{
public:
	virtual FTag* GetTag(const FString& InTagID) = 0;
	virtual FTag* GetTagOfType(const FString& InType) = 0;
	virtual FTag* GetTagOfSubtype(const FString& InTagSubtype) = 0;
	virtual const TArray<const FTag*>& GetTagsArray() const = 0;

	virtual void ParseCLDescription() = 0;
	virtual void ReApplyTags() = 0;
	virtual void ApplyTag(const FString& InTagID) = 0;
	virtual void ApplyTag(FTag& InTag) = 0;
	virtual void UpdateTagsInCL() = 0;
	virtual void RemoveTag(const FString& InTagID) = 0;
	virtual void RemoveTag(FTag& InTag) = 0;
	virtual void SetTagValues(const FString& InTagID, const FString& InValues) = 0;
	virtual void SetTagValues(FTag& InTag, const FString& InValues) = 0;
	virtual void SetTagValues(FTag& InTag, const TArray<FString>& InValues) = 0;

	FTagUpdated OnTagUpdated;
};

Expose_TNameOf(ITagService);
