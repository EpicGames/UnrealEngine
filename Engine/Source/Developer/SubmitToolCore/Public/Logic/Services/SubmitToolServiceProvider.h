// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logic/Services/Interfaces/ISubmitToolService.h"

template<class E>
concept DerivedFromISubmitToolService = std::is_base_of<ISubmitToolService, E>::value;

class FSubmitToolServiceProvider;
DECLARE_MULTICAST_DELEGATE_OneParam(FOnProviderReady, FSubmitToolServiceProvider& /*ServiceProvider*/)

class FSubmitToolServiceProvider final
{

public:
	FSubmitToolServiceProvider()
	{
		OnProviderReady.Broadcast(*this);
	}

	inline static FOnProviderReady OnProviderReady;

	template<DerivedFromISubmitToolService T>
	TSharedPtr<T> GetService()
	{
		FString TypeIdx = TNameOf<T>::GetName();
		if(Services.Contains(TypeIdx))
		{
			return StaticCastSharedPtr<T>(Services[TypeIdx]);
		}

		return nullptr;
	}

	template<DerivedFromISubmitToolService T>
	bool HasService() const
	{
		return Services.Contains(TNameOf<T>::GetName());
	}

	template<DerivedFromISubmitToolService T>
	TSharedPtr<T> GetService(const FString& InName)
	{
		if (Services.Contains(InName))
		{
			return StaticCastSharedPtr<T>(Services[InName]);
		}

		return nullptr;
	}

	template<DerivedFromISubmitToolService T>
	void RegisterService(TSharedRef<T> InService)
	{
		TSharedPtr<ISubmitToolService>& Service = Services.FindOrAdd(TNameOf<T>::GetName());
		ensureAlwaysMsgf(Service == nullptr, TEXT("Service %s was re-registered"), TNameOf<T>::GetName());
		Service = StaticCastSharedPtr<ISubmitToolService>(InService.ToSharedPtr());
	}


	template<DerivedFromISubmitToolService T>
	void RegisterService(TSharedRef<T> InService, const FString& InName)
	{
		TSharedPtr<ISubmitToolService>& Service = Services.FindOrAdd(InName);
		ensureAlwaysMsgf(Service == nullptr, TEXT("Service %s was re-registered"), *InName);
		Service = StaticCastSharedPtr<ISubmitToolService>(InService.ToSharedPtr());
	}

private:
	TMap<FString, TSharedPtr<ISubmitToolService>> Services;
};
