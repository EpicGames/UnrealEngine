// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Class.h"
#include "Internationalization/Regex.h"

#include "ValidatorDefinition.generated.h"

UENUM()
enum class ETaskArea : uint8
{
	None =			0,
	Changelist =	1 << 0,
	LocalFiles =	1 << 1,
	ShelvedFiles =	1 << 2,


	ShelveAndLocalFiles = LocalFiles | ShelvedFiles,
	Everything =	UINT8_MAX

};
ENUM_CLASS_FLAGS(ETaskArea)

USTRUCT()
struct FPathPerExtension
{
	GENERATED_BODY()

	UPROPERTY()
	FString Extension;
	
	UPROPERTY()
	FString Path;
};

USTRUCT()
struct FValidatorDefinition
{
	GENERATED_BODY()

	///
	/// Type of the validator, this is restricted to the classes that are implemented and derive from FValidatorBase
	/// examples include CustomValidator, TagValidator, UBTValidator and others
	/// @see SubmitToolParseConstants
	/// @see FValidatorBase
	///
	UPROPERTY()
	FString Type;

	/// <summary>
	/// This is a mode that will make the validator not report any errors or warnings to the program/user but still log them to telemetry
	/// It doesn't matter if it fails, it will always report as succeed and never block submission.
	/// </summary>
	UPROPERTY()
	bool bIsStealthMode = false;

	///
	///	If true, this validator will be disabled
	/// 
	UPROPERTY()
	bool bIsDisabled = false;

	///
	///	If false, this validator won't have the disable button available
	/// 
	UPROPERTY()
	bool bCanBeDisabledByUser = true;

	///
	///	If false, this validator won't have the Stop button available
	/// 
	UPROPERTY()
	bool bCanBeStoppedByUser = true;

	/// 
	/// Whether the validator is required to allow submission or not, a failing required validation will always block submission
	/// 
	UPROPERTY()
	bool IsRequired = true;
	
	/// 
	/// Whether the validator is required to finish running before allowing submission
	/// 
	UPROPERTY()
	bool bRequireCompleteWhenOptional = false;

	///
	/// Maximum time that a validator will run before being cancelled out
	/// 
	UPROPERTY()
	float TimeoutLimit = -1;
	
	///
	/// Name of this instance of the validator that will be used for display
	/// 
	UPROPERTY()
	FString CustomName;

	///
	/// Regex to test paths to check if this validator should be applied
	/// Replaces IncludeFilesWithExtension & IncludeFilesInDirectory
	/// 
	UPROPERTY()
	FString AppliesToCLRegex;

	/// 
	/// Checks if all these files exists as an additional requirement for this validator to run
	/// 
	UPROPERTY()
	TArray<FString> CheckFileExists;

	///
	/// Optional message to print when the validator doesn't apply to CL to give more context to AppliesToCLRegex
	/// 
	UPROPERTY()
	FString NotApplicableToCLMessage;

	///
	/// Incompatible with AppliesToCLRegex
	/// Files with any of these extensions will be included
	/// 
	UPROPERTY()
	TArray<FString> IncludeFilesWithExtension;

	///
	/// Incompatible with AppliesToCLRegex
	///	Only run this validator for files under this directory
	/// 
	UPROPERTY()
	FString IncludeFilesInDirectory;

	///
	/// Only run this validator for files with this extension under these directories
	/// 
	/// If given, will be used INSTEAD OF IncludeFilesInDirectory for that extension. 
	/// Multiple entries are taken as an OR
	/// Extensions must start with a dot. Only exact matches with the full extension will work (i.e. "filename.foo.bar" WILL NOT match ".bar")
	/// Paths given must be absolute (i.e. start at $(root) ) and are taken as a prefix (i.e. will match any nested subfolders)
	/// (both extensions and path prefix matches are case insensitive)
	/// 
	/// Use as an alternative to AppliesToCLRegex (alongside IncludeFilesInDirectory and IncludeFilesWithExtension) for a human readable finer grained control of which files to select
	/// 
	UPROPERTY()
	TArray<FPathPerExtension> IncludeFilesInDirectoryPerExtension;

	///
	/// Wildcard pattern: only apply this validator to files/path which the specified pattern is matched at any level in the parent hierarchy
	/// i.e: *.uproject -> validator will only include files/paths with a uproject in their parent folder structure
	/// 
	UPROPERTY()
	FString RequireFileInHierarchy;

	///
	/// Wildcard pattern: Opposite of RequireFileInHierarchy this validator will not apply to files that the pattern is matched at any level in the parent hierarchy
	/// i.e: *.uproject -> validator will never include files/paths with a uproject in their parent folder structure
	/// 
	UPROPERTY()
	FString ExcludeWhenFileInHierarchy;
	
	///
	///	This text will be added to the description if this validation passes
	/// 
	UPROPERTY()
	FString ChangelistDescriptionAddendum;

	///
	///	If true, addendum will be appended as long as the validation has been run, even on failure
	/// 
	UPROPERTY()
	bool bAppendAddendumOnAnyResult = false;

	///
	///	Skip this validator when the addendum is already present in the CL description
	/// 
	UPROPERTY()
	bool bSkipWhenAddendumInDescription = false;

	///
	///	Skip this validator when Submit tool is invoked from editor (has the -from-editor cmdline flag)
	///
	UPROPERTY()
	bool bSkipWhenCalledFromEditor = false;

	/// <summary>
	/// This validator will only be applicable when running from editor (-from-editor is present)
	/// </summary>
	UPROPERTY()
	bool bRequiresSTLaunchedFromEditor = false;

	/// <summary>
	/// This validator will only be applicable when NOT running from editor (-from-editor is not present)
	/// </summary>
	UPROPERTY()
	bool bRequiresSTLaunchedStandalone = false;

	///
	///	Skip is forbidden when this text if found in the CL description
	/// 
	UPROPERTY()
	TArray<FString> SkipForbiddenTags;

	///
	/// Skip is forbidden when the CL contains any of these filepatterns
	/// 
	UPROPERTY()
	TArray<FString> SkipForbiddenFiles;

	///
	/// Skip is forbidden when the CL contains any file in the specified streams
	/// 
	UPROPERTY()
	TArray<FString> SkipForbiddenStreams;

	///
	///	List of Validator Ids that needs to succeed before this validator runs
	/// 
	UPROPERTY()
	TArray<FName> DependsOn;

	///
	///	List of execution groups this Validator is part of. Two validators with an execution group in commmon cannot run concurrently
	/// 
	UPROPERTY()
	TArray<FName> ExecutionBlockGroups;

	///
	///	If this Validator runs on files marked for delete
	/// 
	UPROPERTY()
	bool bAcceptDeletedFiles = false;

	///
	///	If this Validator should treat warnings as errors
	/// 
	UPROPERTY()
	bool bTreatWarningsAsErrors = false;
	
	UPROPERTY()
	bool bInvalidatesWhenOutOfDate = false;

	///
	///	If this validator maintains a local cache of results per file between runs on the same CL, used for incremental validations
	/// 
	UPROPERTY()
	bool bUsesIncrementalCache = false;
	
	///
	///	Additional error messages to print when this validation fails
	/// 
	UPROPERTY()
	TArray<FString> AdditionalValidationErrorMessages;

	///
	///	Tooltip when hovering over the Validator
	/// 
	UPROPERTY()
	FString ToolTip;

	///
	///	Area this validator works on, if an area is updated, the validator state will be automatically resetted { Everything, LocalFiles, ShelvedFiles, LocalAndShelvedFiles, Changelist } 
	/// 
	UPROPERTY()
	ETaskArea TaskArea = ETaskArea::Everything;

	UPROPERTY()
	bool bBlocksPreflightStart = true;

	///
	/// Only usable when optional - If this property is specified and the validator has failed, submit tool will show a 
	/// pop up with all the failed validator messages that the user has to confirm through.
	/// 
	UPROPERTY()
	FString PopupMessageWhenFailed = FString();

	///
	///	When enabled this validator failures will be registered in the #FailedValidations tag
	/// 
	UPROPERTY()
	bool bRegisterFailureInCLDescription = false;

	///
	/// If true, this validator will have submit tool perform a shelve operation before running if it's required.
	/// 
	UPROPERTY()
	bool bRequireShelveOperation = false;

	UPROPERTY()
	int32 MinMemoryGBRequired = 0;

	UPROPERTY()
	FString NotEnoughMemoryMessage = TEXT("There is not enough available memory for the estimated consumption. Consider closing other programs and retrying.");

	UPROPERTY()
	bool bAllowSkippingOnMemoryFail = false;

	UPROPERTY()
	FString PreflightTemplate;

	///
	/// Whether to include or not the output warnings in the error summary window
	/// 
	UPROPERTY()
	bool bIncludeWarningsInErrorSummary = true;
};

USTRUCT()
struct FRegexLockdownTrigger
{
	GENERATED_BODY()

	UPROPERTY()
	FString RegexPath;

	TSharedPtr<FRegexPattern> BuiltRegex;

	UPROPERTY()
	FString LockdownId;
};


USTRUCT()
struct FValidatorRunExecutableDefinition : public FValidatorDefinition
{
	GENERATED_BODY()

	UPROPERTY()
	bool bLaunchHidden = true;
	
	UPROPERTY()
	bool bLaunchReallyHidden = true;

	UPROPERTY()
	bool bValidateExecutableExists = true;

	UPROPERTY()
	bool bAllowProcessConcurrency = false;

	///
	///	Path to the executable that this validator runs
	/// 
	UPROPERTY()
	FString ExecutablePath;

	///
	///	Possible Executable paths for this validator to use (user selects)
	/// 
	UPROPERTY()
	TMap<FString, FString> ExecutableCandidates;

	///
	///	When using ExecutableCandidates, default select the newest one
	/// 
	UPROPERTY()
	bool bUseLatestExecutable;

	///
	///	Arguments to pass to the Executable
	/// 
	UPROPERTY()
	FString ExecutableArguments;	
		
	///
	///	These arguments are only added when running submit tool from the editor
	///
	UPROPERTY()
	FString FromEditorOnlyArguments;

	UPROPERTY()
	FString FileInCLArgument;

	///
	///	If specified, list of files will written into a text file and appended to this i.e: a value of "Filelist=" will be 
	/// processed to be "Filelist=Path/To/Intermediate/SubmitTool/FileLists/GUID.txt"
	/// 
	UPROPERTY()
	FString FileListArgument;

	///
	///	When parsing process output, treat these messages as errors
	/// 
	UPROPERTY()
	TArray<FString> ErrorMessages;

	///
	///	When parsing process output, ignore these error messages
	/// 
	UPROPERTY()
	TArray<FString> IgnoredErrorMessages;

	///
	///	When evaluating process exit code, treat these list as a success (defaults to 0)
	/// 
	UPROPERTY()
	TArray<int32> AllowedExitCodes = {0};

	///
	///	Only evaluate validator success using exit code of a process, ignore any output parsing
	/// 
	UPROPERTY()
	bool bOnlyLookAtExitCode = false;

	///
	///	if present, when parsing process output, from this message on, ignore the output
	/// 
	UPROPERTY()
	FString DisableOutputErrorsAnchor;

	///
	///	if Present, when parsing process output, from this message on, parse the output
	/// 
	UPROPERTY()
	FString EnableOutputErrorsAnchor;
	
	///
	///	if present, when parsing process output, from this message on, turn off warnings as errors
	/// 
	UPROPERTY()
	FString DisableWarningsAsErrorsAnchor;

	///
	///	if Present, when parsing process output, from this message on, turn on warnings as errors
	/// 
	UPROPERTY()
	FString EnableWarningsAsErrorsAnchor;

	///
	///	Regex for identifying errors from the output of a process
	/// 
	UPROPERTY()
	FString RegexErrorParsing = TEXT("^(?!.*(?:Display: |Warning: |Log: )).*( error |error:).*$");

	TSharedPtr<FRegexPattern> BuiltRegexError;

	///
	///	Regex for identifying warnings from the output of a process
	/// 
	UPROPERTY()
	FString RegexWarningParsing = TEXT("^(?!.*(?:Display: |Log: )).*( warning |warning:).*$");

	TSharedPtr<FRegexPattern> BuiltRegexWarning;

	///
	///	Additional messages to print when the exitcode is not in the allowed exit codes
	/// 
	UPROPERTY()
	TArray<FString> NonZeroExitCodeErrorMessages;

	UPROPERTY()
	TArray<FRegexLockdownTrigger> LogLockdownTriggers;
};


USTRUCT()
struct FUBTValidatorDefinition : public FValidatorRunExecutableDefinition
{
	GENERATED_BODY()

	UPROPERTY()
	FString Configuration;

	UPROPERTY()
	FString Platform;

	UPROPERTY()
	FString Target;

	UPROPERTY()
	FString ProjectArgument;

	UPROPERTY()
	FString TargetListArgument;

	UPROPERTY()
	TArray<FString> Configurations;

	UPROPERTY()
	TArray<FString> Platforms;

	UPROPERTY()
	TArray<FString> Targets;

	UPROPERTY()
	TArray<FString> StaticAnalysers;

	UPROPERTY()
	FString StaticAnalyserArg;

	UPROPERTY()
	FString StaticAnalyser;

	UPROPERTY()
	bool bUseStaticAnalyser = false;
};

USTRUCT()
struct FPackageDataValidatorDefinition : public FValidatorDefinition
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> ExcludedExtensions;
};

USTRUCT()
struct FJsonValidatorDefinition : public FValidatorDefinition
{
	GENERATED_BODY()

	///
	/// Parse the lines of the json and do not include the ones that match this regex (for custom parsing)
	/// 
	UPROPERTY()
	FString RegexLineExclusion;
};


USTRUCT()
struct FPreflightValidatorDefinition : public FValidatorDefinition
{
	GENERATED_BODY()

	///
	/// The maximum hours since completion that a preflight can be accepted as a success
	/// 
	UPROPERTY()
	int32 MaxPreflightAgeInHours = 12;
};


USTRUCT()
struct FVirtualizationToolDefinition : public FValidatorRunExecutableDefinition
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIncludePackages;

	UPROPERTY()
	bool bIncludeTextPackages;

	UPROPERTY()
	FString BuildCommand;

	UPROPERTY()
	FString BuildCommandArgs;
};

USTRUCT()
struct FEditorCommandletValidatorDefinition : public FValidatorRunExecutableDefinition
{
	GENERATED_BODY()
	
	UPROPERTY()
	FString MainProject;

	UPROPERTY()
	FString EditorRecordsFile;

	UPROPERTY()
	TArray<FString> DisablePlugins;

	UPROPERTY()
	TArray<FString> EnablePlugins;

	UPROPERTY()
	bool bIgnoreUserEditorPrefs = false;
};