// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "CoreMinimal.h"

class FValidatorBase;
class FSubmitToolServiceProvider;
struct FSubmitToolParameters;

class FValidatorFactory
{
public:
	using FValidatorConstructor = TFunction<TSharedRef<FValidatorBase>(
		const FName&,
		const FSubmitToolParameters&,
		const TSharedRef<FSubmitToolServiceProvider>&,
		const FString&)>;

	static FValidatorFactory& Get();

	void Register(const FString& TypeName, FValidatorConstructor InCreator);

	TSharedPtr<FValidatorBase> Create(
		const FString& TypeName,
		const FName& Id,
		const FSubmitToolParameters& Params,
		const TSharedRef<FSubmitToolServiceProvider>& ServiceProvider,
		const FString& Definition) const;

private:
	TMap<FString, FValidatorConstructor> Registry;
};

struct FValidatorRegistrar
{
	FValidatorRegistrar(const FString& TypeName, FValidatorFactory::FValidatorConstructor Creator)
	{
		FValidatorFactory::Get().Register(TypeName, MoveTemp(Creator));
	}
};

#define REGISTER_VALIDATOR_TYPE(TypeNameConstant, ValidatorClass)				\
      static FValidatorRegistrar GRegistrar_##ValidatorClass(					\
          TypeNameConstant,														\
          [](const FName& Id, const FSubmitToolParameters& Params,				\
             const TSharedRef<FSubmitToolServiceProvider>& SP,					\
             const FString& Def) -> TSharedRef<FValidatorBase>					\
          { return MakeShared<ValidatorClass>(Id, Params, SP, Def); });

#define REGISTER_VALIDATOR_TYPE_ALIAS(TypeNameConstant, ValidatorClass, Alias)	\
      static FValidatorRegistrar GRegistrar_##Alias(							\
          TypeNameConstant,														\
          [](const FName& Id, const FSubmitToolParameters& Params,				\
             const TSharedRef<FSubmitToolServiceProvider>& SP,					\
             const FString& Def) -> TSharedRef<FValidatorBase>					\
          { return MakeShared<ValidatorClass>(Id, Params, SP, Def); });