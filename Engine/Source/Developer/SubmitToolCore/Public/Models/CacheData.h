// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "CacheData.generated.h"

USTRUCT()
struct FValidationRecord
{
	GENERATED_BODY()

	UPROPERTY()
	FString ValidatorConfig = FString();

	UPROPERTY()
	FDateTime LastValidationSuccess = FDateTime::MinValue();
};

USTRUCT()
struct FValidationRecords
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FName/*ValidatorId*/, FValidationRecord> LastValidationDateTimes = TMap<FName, FValidationRecord>();
};

USTRUCT()
struct FChangelistCacheData
{
	GENERATED_BODY()
	
	UPROPERTY()
	FDateTime LastShelfIsUpToDate = FDateTime::MinValue();

	UPROPERTY()
	FDateTime LastAccessed = FDateTime::UtcNow();

	UPROPERTY()
	TMap<FString/*InFilePath*/, FValidationRecords> LastFileValidations = TMap<FString, FValidationRecords>();

	UPROPERTY()
	TMap<FString/*FieldId*/, FString/*FieldValue*/> IntegrationFields = TMap<FString, FString>();


	const FDateTime GetLastValidationDate(const FString& InFilePath, const FName& InValidatorId, const FString& InValidatorConfig) const
	{		
		if(LastFileValidations.Contains(InFilePath) && LastFileValidations[InFilePath].LastValidationDateTimes.Contains(InValidatorId) && LastFileValidations[InFilePath].LastValidationDateTimes[InValidatorId].ValidatorConfig.Equals(InValidatorConfig, ESearchCase::IgnoreCase))
		{
			return LastFileValidations[InFilePath].LastValidationDateTimes[InValidatorId].LastValidationSuccess;
		}

		return FDateTime::MinValue();
	}
};

USTRUCT()
struct FCacheFile
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FChangelistCacheData> CLCacheData = TMap<FString, FChangelistCacheData>();
};
