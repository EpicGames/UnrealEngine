// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/ObservableArray.h"
#include "HordeDeploymentData.generated.h"

USTRUCT()
struct FToolDeployment
{
	GENERATED_BODY()

	UPROPERTY()
	FString Id;

	UPROPERTY()
	FString Version;

	UPROPERTY()
	FString State;

	UPROPERTY()
	int Progress;

	UPROPERTY()
	FString StartedAt;

	UPROPERTY()
	FString Duration;

	UPROPERTY()
	FString RefName;

	UPROPERTY()
	FString Locator;
};

USTRUCT()
struct FDeploymentList
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FToolDeployment> Deployments;

};