// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Parameters/SubmitToolParameters.h"

class FIntegrationOptionBase
{
public:
	FIntegrationOptionBase(const FJiraIntegrationField& InFieldDefinition) : FieldDefinition(InFieldDefinition)
	{}
	virtual ~FIntegrationOptionBase() {}

	FJiraIntegrationField FieldDefinition;

	bool bInvalid = false;

	virtual bool GetJiraValue(FString& OutValue) = 0;
	virtual void SetValue(const FString& InValue) = 0;
};


class FIntegrationEmptyOption : public FIntegrationOptionBase
{		
public:
	FIntegrationEmptyOption(const FJiraIntegrationField& InFieldDefinition) : FIntegrationOptionBase(InFieldDefinition)
	{}

	virtual bool GetJiraValue(FString& OutValue) override
	{
		return false;
	}

	virtual void SetValue(const FString& InValue)
	{
	}
};

class FIntegrationBoolOption : public FIntegrationOptionBase
{
public:
	FIntegrationBoolOption(const FJiraIntegrationField& InFieldDefinition, bool InDefaultValue = false) : 
		FIntegrationOptionBase(InFieldDefinition),
		Value(InDefaultValue)
	{}
	bool Value = false;

	virtual bool GetJiraValue(FString& OutValue) override
	{
		if(FieldDefinition.JiraValues.Num() == 0)
		{
			return false;
		}
		else if(FieldDefinition.JiraValues.Num() == 1)
		{
			if(Value)
			{
				OutValue = FieldDefinition.JiraValues[0];
			}

			return Value;
		}
		else
		{
			OutValue = Value ? FieldDefinition.JiraValues[0] : FieldDefinition.JiraValues[1];
			return true;
		}
	}

	virtual void SetValue(const FString& InValue) override
	{
		if(InValue.Equals(FieldDefinition.JiraValues[0], ESearchCase::IgnoreCase))
		{
			Value = true;
		}
	}
};

class FIntegrationTextOption : public FIntegrationOptionBase
{
public:
	FIntegrationTextOption(const FJiraIntegrationField& InFieldDefinition) : 
		FIntegrationOptionBase(InFieldDefinition),
		Value(InFieldDefinition.Default)
	{}

	FString Value;

	virtual bool GetJiraValue(FString& OutValue) override
	{
		OutValue = Value;
		return !OutValue.IsEmpty();
	}

	virtual void SetValue(const FString& InValue) override
	{
		Value = InValue;
	}
};

class FIntegrationComboOption : public FIntegrationTextOption
{
public:
	FIntegrationComboOption(const FJiraIntegrationField& InFieldDefinition) : FIntegrationTextOption(InFieldDefinition)
	{
		for(const FString& OptionValue : FieldDefinition.JiraValues)
		{
			ComboValues.Add(MakeShared<FString>(OptionValue));
		}
	}
	TArray<TSharedPtr<FString>> ComboValues;
};