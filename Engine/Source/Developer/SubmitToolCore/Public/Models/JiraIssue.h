// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "JiraIssue.generated.h"

USTRUCT()
struct FJiraIssue
{
	GENERATED_BODY()

	FJiraIssue() = default;
	FJiraIssue(const FString& InKey, const FString& InSummary, const FString& InLink, const FString& InDescription, const FString& InPriority, const FString& InStatus, const FString& InIssueType) :
		Key(InKey),
		Summary(InSummary),
		Link(InLink),
		Description(InDescription),
		Priority(InPriority),
		Status(InStatus),
		IssueType(InIssueType)
	{
	}
	UPROPERTY()
	FString Key;

	UPROPERTY()
	FString Summary;

	UPROPERTY()
	FString Link;

	UPROPERTY()
	FString Description;

	UPROPERTY()
	FString Priority;

	UPROPERTY()
	FString Status;

	UPROPERTY()
	FString IssueType;
};
