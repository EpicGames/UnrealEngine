// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "JiraToken.generated.h"

USTRUCT()
struct FJiraToken
{
	GENERATED_BODY()

	UPROPERTY()
	FString Access_Token;
	UPROPERTY()
	int32 Expires_In = 0;
	UPROPERTY()
	FString Refresh_Token;
};
