// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Hash/Fnv.h"
#include "LockdownData.generated.h"

UENUM()
enum class ELockdownType : int32
{
	None = 0,
	Controlled,
	Hardcore
};

struct FCaseInsensitiveStringKeyFuncs : DefaultKeyFuncs<FString, false>
{
	static FORCEINLINE bool Matches(const FString& A, const FString& B)
	{
		return A.Equals(B, ESearchCase::IgnoreCase);
	}
	static FORCEINLINE uint32 GetKeyHash(const FString& Key)
	{
		const FString Lower = Key.ToLower();
		return UE::HashStringFNV1a32(MakeStringView(*Lower));
	}
};

struct FSubmitToolLockdownData
{
	TSet<FString, FCaseInsensitiveStringKeyFuncs> GroupNames;
	TSet<FString, FCaseInsensitiveStringKeyFuncs> GroupMessages;
	ELockdownType LockdownType = ELockdownType::None;
	TMap<FString, TSet<FString, FCaseInsensitiveStringKeyFuncs>> RequiredTags;
	bool bIsAllowlisted = true;

	void Append(FSubmitToolLockdownData&& OtherLockdown)
	{
		if (OtherLockdown.LockdownType == ELockdownType::None)
		{
			return;
		}

		GroupNames.Append(MoveTemp(OtherLockdown.GroupNames));
		GroupMessages.Append(MoveTemp(OtherLockdown.GroupMessages));
		for (auto It = OtherLockdown.RequiredTags.CreateIterator(); It; ++It)
		{
			if (RequiredTags.Contains(It->Key))
			{
				RequiredTags[It->Key].Append(MoveTemp(It->Value));
				It.RemoveCurrent();
			}
		}
		RequiredTags.Append(MoveTemp(OtherLockdown.RequiredTags));
		bIsAllowlisted &= OtherLockdown.bIsAllowlisted;
		if (LockdownType < OtherLockdown.LockdownType)
		{
			LockdownType = OtherLockdown.LockdownType;
		}
	}
};
