// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PreflightData.generated.h"

UENUM()
enum class EPreflightOutcome : uint8
{
	Unspecified = 0,
	Failure,
	Warnings,
	Success
};

UENUM()
enum class EPreflightState : uint8
{
	Unspecified = 0,
	Ready,
	Running,
	Skipped,
	Completed
};

template<typename PreflightEnum>
PreflightEnum ParsePreflightEnum(const FString& String)
{
	UEnum* PreflightOutcomeEnum = StaticEnum<PreflightEnum>();
	check(PreflightOutcomeEnum);

	int64 EnumValue = PreflightOutcomeEnum->GetValueByNameString(String);
	return (PreflightEnum)(EnumValue >= 0 ? EnumValue : 0);
}

USTRUCT()
struct FPreflightResultData
{
	GENERATED_BODY()

	UPROPERTY()
	EPreflightState State = EPreflightState::Unspecified;
	UPROPERTY()
	EPreflightOutcome Outcome = EPreflightOutcome::Unspecified;
	UPROPERTY()
	TArray<FString> Errors;

	bool WasSuccessful() const
	{
		return State == EPreflightState::Completed && Outcome == EPreflightOutcome::Success && Errors.Num() == 0;
	}

	bool operator==(const FPreflightResultData& other) const
	{
		return State == other.State && Outcome == other.Outcome;
	}

	bool operator!=(const FPreflightResultData& other) const
	{
		return !(*this == other);
	}
};

USTRUCT()
struct FPFStep
{
	GENERATED_BODY()
				
	UPROPERTY()
	FString ID;
	UPROPERTY()
	FString State;
	UPROPERTY()
	FString Outcome;
	UPROPERTY()
	FString Error;
	UPROPERTY()
	FString RetryByUser;


	bool operator==(const FPFStep& other) const = default;
	bool operator!=(const FPFStep& other) const = default;
};

USTRUCT()
struct FPFBatch
{
	GENERATED_BODY()

	UPROPERTY()
	FString ID;
	UPROPERTY()
	FString State;
	UPROPERTY()
	TArray<FPFStep> Steps;


	bool operator==(const FPFBatch& other) const = default;
	bool operator!=(const FPFBatch& other) const = default;
};

USTRUCT()
struct FPreflightData
{
	GENERATED_BODY()
	
	UPROPERTY()
	FString ID;
	UPROPERTY()
	FString Name;
	UPROPERTY()
	FString TemplateId;
	UPROPERTY()
	FDateTime CreateTime;
	UPROPERTY()
	FDateTime UpdateTime;
	UPROPERTY()
	TArray<FPFBatch> Batches;

	FPreflightResultData CachedResults;

	void RecalculateCachedResults()
	{
		CachedResults = FPreflightResultData();

		TMap<EPreflightState, int32> StateCounts;
		TMap<EPreflightOutcome, int32> OutcomeCounts;
		for (const FPFBatch& Batch : Batches)
		{
			for (const FPFStep& Step : Batch.Steps)
			{
				if (!Step.RetryByUser.IsEmpty())
				{
					// This step has been retried. Ignore it.
					continue;
				}

				StateCounts.FindOrAdd(ParsePreflightEnum<EPreflightState>(Step.State), 0)++;
				OutcomeCounts.FindOrAdd(ParsePreflightEnum<EPreflightOutcome>(Step.Outcome), 0)++;

				if (!Step.Error.IsEmpty() && Step.Error != "None")
				{
					CachedResults.Errors.Add(Step.Error);
				}
			}
		}

		// Build the results.
		const bool bHaveAllStepsFinished = (StateCounts.Contains(EPreflightState::Completed) || StateCounts.Contains(EPreflightState::Skipped)) 
										&& !StateCounts.Contains(EPreflightState::Running) 
										&& !StateCounts.Contains(EPreflightState::Ready);
		CachedResults.State = bHaveAllStepsFinished ? EPreflightState::Completed : EPreflightState::Running;
		CachedResults.Outcome = EPreflightOutcome::Unspecified;

		if (CachedResults.State == EPreflightState::Completed)
		{
			auto testOutcome = [this, &OutcomeCounts](EPreflightOutcome OutcomeToTest)
			{
				if (OutcomeCounts.Contains(OutcomeToTest) && OutcomeCounts[OutcomeToTest] > 0)
				{
					CachedResults.Outcome = OutcomeToTest;
				}
			};

			// Override the outcome from the steps in this order.
			testOutcome(EPreflightOutcome::Success);
			testOutcome(EPreflightOutcome::Warnings);
			testOutcome(EPreflightOutcome::Failure);
			testOutcome(EPreflightOutcome::Unspecified);
		}
	}


	bool operator==(const FPreflightData& other) const = default;
	bool operator!=(const FPreflightData& other) const = default;
};

USTRUCT()
struct FPreflightList
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FPreflightData> PreflightList;

	void Initialize()
	{
		for(FPreflightData& PFData : PreflightList)
		{
			PFData.RecalculateCachedResults();
		}
	}

	bool operator==(const FPreflightList& other) const = default;
	bool operator!=(const FPreflightList& other) const = default;
};

using FOnPreflightDataUpdated = TMulticastDelegate<void(const TUniquePtr<FPreflightList>&, const TMap<FString, FPreflightData>&)>;
