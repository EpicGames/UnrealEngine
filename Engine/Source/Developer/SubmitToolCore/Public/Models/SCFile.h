// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Misc/Paths.h"
#include "SCFile.generated.h"

UENUM()
enum ESCFileState : int64
{
	Edit,
	Add,
	Delete,
	Integrate,
	Unknown
};

class FSCFile final : public TSharedFromThis<class FSCFile, ESPMode::ThreadSafe>
{
public:
	FSCFile(const FString& InFilename, const FString& InDepotPath, ESCFileState InState)
	: Filename(InFilename)
	, DepotPath(InDepotPath)
	, State(InState)
	{ 
		FPaths::NormalizeFilename(Filename);
		FPaths::NormalizeFilename(DepotPath);
	}
	~FSCFile() {}

	const FString& GetFilename() const { return Filename; }
	const FString& GetDepotPath() const { return DepotPath; }
	ESCFileState GetState() const { return State; }
	void UpdateState(ESCFileState NewState) { State = NewState; }
	void UpdateFilename(const FString& NewFilename) { Filename = NewFilename; }

	bool IsAdded() const { return State == ESCFileState::Add; };
	bool IsCheckedOut() const { return State == ESCFileState::Edit || State == ESCFileState::Unknown; }
	bool IsIntegrate() const { return State == ESCFileState::Integrate; }	
	bool IsDeleted() const { return State == ESCFileState::Delete; }

private:
	FString Filename;
	FString DepotPath;
	ESCFileState State;
};

using FSCFileRef = TSharedRef<FSCFile, ESPMode::ThreadSafe>;
