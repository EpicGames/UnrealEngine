// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Logging/SubmitToolLog.h"
#include "SubmitToolUserPrefs.generated.h"

USTRUCT()
struct FSubmitToolUserPrefs
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FString> ValidatorOptions;
	
	UPROPERTY()
	bool bCloseOnSubmit = true;

	UPROPERTY()
	bool bOpenJiraOnSubmit = true;
	
	UPROPERTY()
	bool bExpandFilesInCL = false;
	
	UPROPERTY()
	bool bWindowMaximized = false;

	UPROPERTY()
	bool bAutoUpdatePreflight = false;
	
	UPROPERTY()
	bool bAutoCreateShelf = false;
	
	UPROPERTY()
	bool bAutoStompShelf = false;

	UPROPERTY()
	FVector2D WindowPosition = FVector2D();

	UPROPERTY()
	FVector2D WindowSize = FVector2D();
	
	UPROPERTY()
	float TagSectionSize = FLT_MAX;

	UPROPERTY()
	float P4SectionSize = 34;
	
	UPROPERTY()
	float ValidatorSectionSize = 22;
	
	UPROPERTY()
	float LogSectionSize = 46;

	UPROPERTY()
	TMap<FName, bool> UISectionExpandState =
	{};

	UPROPERTY()
	bool bAutoUpdate = true;
	
	UPROPERTY()
	bool bAppendAtForP4Users = true;

	UPROPERTY()
	bool bAppendAtForP4Groups = true;

	virtual ~FSubmitToolUserPrefs();

	static FSubmitToolUserPrefs* Get()
	{
		if(Instance == nullptr)
		{
			UE_LOGF(LogSubmitTool, Error, "SubmitToolUserPrefs is not valid");
		}

		return Instance;
	}

	static TUniquePtr<FSubmitToolUserPrefs> Initialize(const FString& InFilePath);
	private:

	static FString FilePath;
	static FSubmitToolUserPrefs* Instance;
};
