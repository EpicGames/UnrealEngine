// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "SwarmReview.generated.h"

USTRUCT()
struct FSwarmReviewVote
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Value;

	UPROPERTY()
	int32 Version;

	UPROPERTY()
	bool IsStale;
};

USTRUCT()
struct FSwarmReviewParticipant
{
	GENERATED_BODY()

	UPROPERTY()
	FSwarmReviewVote Vote;
};

USTRUCT()
struct FSwarmReview
{
	GENERATED_BODY()

	UPROPERTY()
	int32 Id;

	UPROPERTY()
	FString State;

	UPROPERTY()
	FString Author;

	UPROPERTY()
	TMap<FString, FSwarmReviewParticipant> Participants;
};

USTRUCT()
struct FSwarmReviewCollection
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FSwarmReview> Reviews;
};