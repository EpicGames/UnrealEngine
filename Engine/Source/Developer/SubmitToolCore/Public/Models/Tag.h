// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TagDefinition.h"
#include "SCFile.h"

class FTag;

DECLARE_MULTICAST_DELEGATE_OneParam(FTagUpdated, const FTag&)

enum class ETagState : uint8
{
	Unchecked = 0,
	Failed,
	Success
};

class FTag {
public:
	FTag(const FTagDefinition& def) : FTag(def, std::numeric_limits<size_t>::max())
	{}

	FTag(const FTagDefinition& def, size_t start);

	FTag(const FTag& InOther) : FTag(InOther.Definition, InOther.StartPos)
	{
		TagValues = InOther.TagValues;
		TagKeyValue = InOther.TagKeyValue;

		ValidationState = ETagState::Unchecked;
		bIsDirty = InOther.bIsDirty;
		OnTagUpdated = InOther.OnTagUpdated;
	}

	FTag(FTag && InOther) : FTag(InOther.Definition, InOther.StartPos)
	{
		TagValues = MoveTemp(InOther.TagValues);
		TagKeyValue = MoveTemp(InOther.TagKeyValue);

		ValidationState = ETagState::Unchecked;
		bIsDirty = MoveTemp(InOther.bIsDirty);
		OnTagUpdated = MoveTemp(InOther.OnTagUpdated);
	}

	FTag& operator=(const FTag& InOther) = delete;
	FTag& operator=(FTag&& InOther) = delete;

	FString GetFullTag() const;
	bool ParseTag(const FString& source);
	
	void SetValues(const FString& valuesText);
	FString GetValuesText() const;

	void SetValues(const TArray<FString>& InValues); 
	void SetValues(TArray<FString>&& InValues );
	const TArray<FString> GetValues(bool bIncludeInactive = false) const;

	const FTagValidationConfig& GetCurrentValidationConfig(const TArray<FSCFileRef>& InDepotPaths) const;

	const ETagState GetTagState() const
	{
		return ValidationState;
	}

	void SetTagState(ETagState InState) const
	{
		ValidationState = InState;
	}

	bool IsEnabled() const
	{
		return StartPos != std::numeric_limits<size_t>::max();
	}

	FTagUpdated OnTagUpdated;

	bool bIsDirty = false;
	const FTagDefinition& Definition;
	size_t StartPos = std::numeric_limits<size_t>::max();
	size_t LastSize = std::numeric_limits<size_t>::max();
private:
	void Reset()
	{
		ValidationState = ETagState::Unchecked;
		bIsDirty = false;
		StartPos = std::numeric_limits<size_t>::max();
		LastSize = std::numeric_limits<size_t>::max();
		TagValues.Empty();
		OnTagUpdated.Broadcast(*this);
	}

	TArray<TCHAR const*> Delimiters;
	TArray<FString> SplittedDelims;
	TArray<FString> TagValues;
	FString TagKeyValue;
	mutable ETagState ValidationState;
};
