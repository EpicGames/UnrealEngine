// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "TagDefinition.generated.h"


USTRUCT()
struct FTagValidationConfig
{
	GENERATED_BODY()

	FTagValidationConfig() = default;

	UPROPERTY()
	FString RegexValidation = FString();

	UPROPERTY()
	FString RegexErrorMessage = FString();

	UPROPERTY()
	bool bIsMandatory = false;
};

USTRUCT()
struct FTagValidationOverride
{
	GENERATED_BODY()
	
	UPROPERTY()
	FString RegexPath;

	UPROPERTY()
	FTagValidationConfig ConfigOverride;
};

USTRUCT()
struct FTagDefinition
{
	GENERATED_BODY()

	FTagDefinition() = default;

	inline const FString& GetTagId() const { return TagId; }

	///
	/// Tag that is inserted/scanned into the CL description
	/// 
	UPROPERTY()
	FString TagId;

	///
	/// 
	/// 
	UPROPERTY()
	FString RegexParseOverride;

	///
	///	Name of the tag to show in the UI Widget
	/// 
	UPROPERTY()
	FString TagLabel;

	///
	/// Tooltip to show in the UI Widget
	/// 
	UPROPERTY()
	FString ToolTip = FString(TEXT("There is no defined documentation for this tag."));


	///
	/// URL to link the documentation to
	/// 
	UPROPERTY()
	FString DocumentationUrl;


	///
	/// Input type of the UI widget, available types:
	/// - FreeText: tag with a text input
	/// - Boolean: tag without any values
	/// - MultiSelect: tag with text input and a list of dropdown checkboxes (needs SelectValues)
	/// - PerforceUser: tag with User and Group buttons that are fetched from p4 users and groups
	/// - JiraIssue: tag with a jira button to a list of tickets that get fetched from jira (needs other config for connection)
	/// 
	UPROPERTY()
	FString InputType;

	///
	/// Used for specifying different subtypes of tag widgets:
	/// - With PerforceUser InputType:
	///		- SwarmApproved: this tag will automatically query Swarm to apply the values of the usernames that upvoted the review
	///		- Swarm: this tag will have functionality of requesting/opening a swarm review with the username/groups that are specified.
	/// - With FreeText InputType:
	///		- Preflight: this tag will fetch information from horde regarding a preflight and allow requesting one.
	/// 
	UPROPERTY()
	FString InputSubType;

	///
	/// Specifies the standard delimitation of values in a tag (#test 1, 2, 3 has values {1,2,3} with the ", " removed)
	/// 
	UPROPERTY()
	FString ValueDelimiter = TEXT(", ");	

	///
	/// Minimum amount of values this tag can have
	/// 
	UPROPERTY()
	int32 MinValues = 0;

	///
	/// Maximum amount of values this tag can have
	/// 
	UPROPERTY()
	int32 MaxValues = UINT8_MAX;

	///
	/// Base Validation properties for this tag
	/// 
	UPROPERTY()
	FTagValidationConfig Validation = FTagValidationConfig();	

	///
	/// List of validation requirements override according to file paths, useful for changing tags requirements when certain files are in the CL
	/// 
	UPROPERTY()
	TArray<FTagValidationOverride> ValidationOverrides;

	///
	/// Sorting order of the tag
	/// 
	UPROPERTY()
	int32 OrdinalOverride = 0;

	///
	/// Whether this tag is in use or not
	/// 
	UPROPERTY()
	bool bIsDisabled = false;

	///
	/// Options for MultiSelect InputType
	/// 
	UPROPERTY()
	TArray<FString> SelectValues;

	///
	/// Only for PerforceUser InputType: allow list to filter out anything that doesn't contain a value in this list, useful for filtering by e-mail domains
	/// 
	UPROPERTY()
	TArray<FString> Filters;
};
