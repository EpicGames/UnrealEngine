// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Models/Tag.h"
#include "Logic/Validators/ValidatorDefinition.h"
#include "UObject/Class.h"
#include "Configuration/Configuration.h"
#include "SubmitToolParameters.generated.h"

USTRUCT()
struct FDocumentationLink
{
	GENERATED_BODY()

	UPROPERTY()
	FString Text;
	
	UPROPERTY()
	FString Tooltip;

	UPROPERTY()
	FString Link;
};

USTRUCT()
struct FGeneralParameters 
{
	GENERATED_BODY()

	UPROPERTY()
	FString NewChangelistMessage = TEXT("Submit Tool generated changelist from default");

	UPROPERTY()
	TArray<FString> ForbiddenDescriptions = 
	{
		TEXT("Submit Tool generated changelist from default"),
		TEXT("<saved by perforce>")
	};

	UPROPERTY()
	FString CacheFile = TEXT("$(localappdata)/SubmitTool/SubmitToolCache.cache");

	UPROPERTY()
	uint8 InvalidateCacheHours = 36;
	
	UPROPERTY()
	TArray<FDocumentationLink> HelpLinks;
	
	UPROPERTY()
	uint8 EarlySubmitHour24 = 6;

	UPROPERTY()
	uint8 LateSubmitHour24 = 16;

	UPROPERTY()
	FString OutOfHoursSubmitMessage = TEXT("**** It's $(DateTime) local time, be mindful of submitting late and going away, please remain alert and available for a reasonable period of time (2-3 hours) in case your changes cause issues ****");

	UPROPERTY()
	TArray<FString> GroupsToExclude;
	
	UPROPERTY()
	TMap<FString, FString> PathOverrides;
	
	UPROPERTY()
	TMap<FString, FString> PathOverridesInSourceBuild;
	
	UPROPERTY()
	TArray<FString> AutomaticChangelistTags;

	UPROPERTY()
	TArray<FString> SyncOnStartupPaths;

	UPROPERTY()
	FString DescriptionGenProcess;

	UPROPERTY()
	FString DescriptionGenTooltip;

	UPROPERTY()
	FString DescriptionGenArgs;

	UPROPERTY()
	FString DescriptionGenParsing;

	UPROPERTY()
	FString DescriptionGenDocumentationUrl;
};

USTRUCT()
struct FJiraParameters
{
	GENERATED_BODY()

	UPROPERTY()
	bool bDisableJira = false;

	UPROPERTY()
	FString OAuthClientId;

	UPROPERTY()
	FString OAuthClientSecret;

	UPROPERTY()
	FString CloudID;

	UPROPERTY()
	FString ServerAddress;
	
	UPROPERTY()
	int64 ServiceDeskID;
	
	UPROPERTY()
	int64 RequestFormID;
	
	UPROPERTY()
	FString ServiceDeskToken;

	UPROPERTY()
	FString SwarmUrlField;

	UPROPERTY()
	FString RequestorField;

	UPROPERTY()
	FString PreflightField;

	UPROPERTY()
	FString JiraField;
	
	UPROPERTY()
	FString StreamField;
};

USTRUCT()
struct FPreflightAdditionalTask
{
	GENERATED_BODY()

	UPROPERTY()
	FString RegexPath;

	UPROPERTY()
	FString TaskId;
};

USTRUCT()
struct FPreflightTemplateDefinition
{
	GENERATED_BODY()

	UPROPERTY()
	FString RegexPath;

	UPROPERTY()
	FString Template;

	UPROPERTY()
	TArray<FPreflightAdditionalTask> AdditionalTasks;
};

USTRUCT()
struct FTelemetryParameters
{
	GENERATED_BODY()

	UPROPERTY()
	FString Url;
	UPROPERTY()
	FString Instance;
	UPROPERTY()
	FString RegexStreamAllowlist;
};

UENUM()
enum class EFieldType
{
	Bool = 0,
	Text = 1,
	MultiText = 2,
	Combo = 3,
	PerforceUser = 4,

	UILabel = 99,
	UISpace = 100
};

UENUM()
enum class EJiraFieldType
{
	Object = 0,
	Array = 1,
	String = 2
};

USTRUCT()
struct FJiraIntegrationField
{
	GENERATED_BODY()

	UPROPERTY()
	FString Id;
	UPROPERTY()
	FString Name;
	UPROPERTY()
	FString LabelDisplay;
	UPROPERTY()
	TArray<FString> JiraValues;
	UPROPERTY()
	EFieldType Type;
	UPROPERTY()
	EJiraFieldType JiraType;
	UPROPERTY()
	FString Default;
	UPROPERTY()
	TArray<FString> DependsOn;
	UPROPERTY()
	FString DependsOnValue;
	UPROPERTY()
	TArray<FString> ValidationGroups;
	UPROPERTY()
	bool bRequiredValue;
	UPROPERTY()
	FString Tooltip;
};

USTRUCT()
struct FIntegrationParameters
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FJiraIntegrationField> Fields;
	UPROPERTY()
	TArray<FString> OneOfValidationGroups;
};

USTRUCT()
struct FCopyLogParameters
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> LogsToCollect;
};

USTRUCT()
struct FP4LockdownParameters
{
	GENERATED_BODY()

	UPROPERTY()
	TMap<FString, FString> ConfigPaths;
	
	UPROPERTY()
	TArray<FString> AdditionalHardlockedPaths;
	
	UPROPERTY()
	TMap<FString, FString> DynamicConfigPaths;	
};

USTRUCT()
struct FOAuthTokenParams
{
	GENERATED_BODY()

	UPROPERTY()
	FString OAuthTokenTool;

	UPROPERTY()
	FString OAuthArgs;

	UPROPERTY()
	FString OAuthFile;
};

USTRUCT()
struct FIncompatibleFilesGroup
{
	GENERATED_BODY()

	UPROPERTY()
	TArray<FString> FileGroups;

	UPROPERTY()
	bool bIsError = false;

	UPROPERTY()
	FString Title = TEXT("Files from different groups");

	UPROPERTY()
	FString MessageFormat = TEXT("You are submitting files in the same CL in these locations:\n{Groups}");

	const FString GetMessage() const
	{
		TArray<FString> SubstFileGroups;
		for(const FString& Group : FileGroups)
		{
			SubstFileGroups.Add(FConfiguration::Substitute(Group));
		}

		FStringFormatNamedArguments Args = {
			{
				TEXT("Groups"),
				FString::Join(SubstFileGroups, TEXT("\n"))
			}
		};

		return FString::Format(*MessageFormat, Args);
	};
};

USTRUCT()
struct FIncompatibleFilesParams
{
	GENERATED_BODY()
	
	UPROPERTY()
	TArray<FIncompatibleFilesGroup> IncompatibleFileGroups;
};


USTRUCT()
struct FHordeParameters
{
	GENERATED_BODY()

	UPROPERTY()
	FString HordeServerAddress;
	UPROPERTY()
	FString StartPreflightURLFormat;
	UPROPERTY()
	FString FindPreflightURLFormat;
	UPROPERTY()
	FString FindSinglePreflightURLFormat;
	UPROPERTY()
	float FetchPreflightEachSeconds = 180;
	UPROPERTY()
	float FetchPreflightEachSecondsWhenInProgress = 90;
	UPROPERTY()
	FString DefaultPreflightTemplate;
	UPROPERTY()
	TArray<FPreflightTemplateDefinition> Definitions;
};

USTRUCT()
struct FAutoUpdateParameters
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIsAutoUpdateOn = false;

	UPROPERTY()
	FString AutoUpdateScript;

	UPROPERTY()
	FString LocalAutoUpdateScript;

	UPROPERTY()
	FString AutoUpdateCommand;

	UPROPERTY()
	FString AutoUpdateArgs;

	UPROPERTY()
	FString DeployIdFilePath;

	UPROPERTY()
	FString LocalDownloadZip;

	UPROPERTY()
	FString LocalVersionFile;
};

struct FSubmitToolParameters
{
	FGeneralParameters GeneralParameters;
	FTelemetryParameters Telemetry;
	TArray<FTagDefinition> AvailableTags;
	TMap<FString, FString> Validators;
	TMap<FString, FString> PresubmitOperations;
	FJiraParameters JiraParameters;
	FIntegrationParameters IntegrationParameters;
	FCopyLogParameters CopyLogParameters;
	FP4LockdownParameters P4LockdownParameters;
	FOAuthTokenParams OAuthParameters;
	FIncompatibleFilesParams IncompatibleFilesParams;
	FHordeParameters HordeParameters;
	FAutoUpdateParameters AutoUpdateParameters;
};
