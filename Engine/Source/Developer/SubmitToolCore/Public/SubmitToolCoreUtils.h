// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"

class FString;

class FSubmitToolCoreUtils
{
public:
	static FString GetLocalAppDataPath();

	static void CopyDiagnosticFilesToClipboard(TConstArrayView<FString> Files);

	static bool IsFileInHierarchy(const FString& InWildcard, const FString& InPath);

	template <int32 BufferSize = 256, typename RangeType>
	static FString StringBuilderJoin(const RangeType& InRange, const FString&& Delimiter)
	{
		TStringBuilder<BufferSize> Builder;
		Builder.Join(InRange, Delimiter);
		return Builder.ToString();
	}

private:
	static TMap<FString, TMap<bool, TSet<FString>>> HierarchyWildcardsCache;
};
