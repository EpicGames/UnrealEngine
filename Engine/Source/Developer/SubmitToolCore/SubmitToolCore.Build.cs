// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
public class SubmitToolCore : ModuleRules
{
	public SubmitToolCore(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"TargetPlatform"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AppFramework",
				"Core",
				"CoreUObject",
				"Json",
				"OutputLog"
			}
		);
	}
}
