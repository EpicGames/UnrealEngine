// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "TestCommon/Expectations.h"

#include "CommandLine/CmdLineParameter.h"

TEST_CASE("SubmitTool::Core::CommandLine::Parameter::IsValid", "[SubmitToolCore][CommandLine][Parameter]")
{
	const TSharedPtr<FCmdLineParameter> TestParam = MakeShared<FCmdLineParameter>(
		TEXT("TestParam"),
		true,
		TEXT("Test Parameter"),
		false,
		[](const FString& InValue)
		{
			return !InValue.IsEmpty();
		});

	CHECK(TestParam->IsValid(TEXT("NonEmpty")));
	CHECK_FALSE(TestParam->IsValid(TEXT("")));
}

TEST_CASE("SubmitTool::Core::CommandLine::Parameter::CustomParse", "[SubmitToolCore][CommandLine][Parameter]")
{
	const FString ExpectedParseValue = TEXT("New Test Parameter");
	const TSharedPtr<FCmdLineParameter> TestParam = MakeShared<FCmdLineParameter>(
		TEXT("TestParam"),
		true,
		TEXT("Test Parameter"),
		false,
		nullptr,
		[&ExpectedParseValue](FString& OutValue)
		{
			OutValue = ExpectedParseValue;
		});

	FString ToBeParsed = TEXT("Not Yet Parsed");
	
	CHECK_NOT_EQUALS(TEXT("CustomParser before parse"), ToBeParsed, ExpectedParseValue);
	TestParam->CustomParse(ToBeParsed);
	CHECK_EQUALS(TEXT("CustomParser after parse"), ToBeParsed, ExpectedParseValue);
}

#endif