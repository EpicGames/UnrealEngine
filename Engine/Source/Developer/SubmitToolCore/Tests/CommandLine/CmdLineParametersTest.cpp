// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_LOW_LEVEL_TESTS

#include "TestHarness.h"
#include "TestCommon/Expectations.h"

#include "Logging/LogScopedVerbosityOverride.h"
#include "Logging/LogVerbosity.h"
#include "Misc/CommandLine.h"

#include "CommandLine/CmdLineParameters.h"
#include "Logging/SubmitToolLog.h"


GROUP_AFTER_ALL("CmdLineParameters")
{
	FCommandLine::Set(TEXT(""));
}

GROUP_TEST_CASE("CmdLineParameters", "SubmitTool::Core::CommandLine::Parameters::Validate Empty Command Line", "[SubmitToolCore][CommandLine][Parameters]")
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSubmitTool, ELogVerbosity::NoLogging);

	FCommandLine::Set(TEXT(""));
	CHECK_FALSE(FCmdLineParameters::Get().ValidateParameters());
}

GROUP_TEST_CASE("CmdLineParameters", "SubmitTool::Core::CommandLine::Parameters::Validate with Server", "[SubmitToolCore][CommandLine][Parameters]")
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSubmitTool, ELogVerbosity::NoLogging);

	FCommandLine::Set(TEXT("-server perforce:9999"));
	CHECK_FALSE(FCmdLineParameters::Get().ValidateParameters());
}

GROUP_TEST_CASE("CmdLineParameters", "SubmitTool::Core::CommandLine::Parameters::Validate with Server and Client", "[SubmitToolCore][CommandLine][Parameters]")
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSubmitTool, ELogVerbosity::NoLogging);

	FCommandLine::Set(TEXT("-server perforce:9999 -client mycomputer"));
	CHECK_FALSE(FCmdLineParameters::Get().ValidateParameters());
}

GROUP_TEST_CASE("CmdLineParameters", "SubmitTool::Core::CommandLine::Parameters::Validate with Server and Client and User", "[SubmitToolCore][CommandLine][Parameters]")
{
	LOG_SCOPE_VERBOSITY_OVERRIDE(LogSubmitTool, ELogVerbosity::NoLogging);

	FCommandLine::Set(TEXT("-server perforce:9999 -client mycomputer -user testuser"));
	CHECK_FALSE(FCmdLineParameters::Get().ValidateParameters());
}

GROUP_TEST_CASE("CmdLineParameters", "SubmitTool::Core::CommandLine::Parameters::Validate Passing", "[SubmitToolCore][CommandLine][Parameters]")
{
	FCommandLine::Set(TEXT("-server perforce:9999 -client mycomputer -user testuser -cl 1234567"));
	CHECK(FCmdLineParameters::Get().ValidateParameters());
}

GROUP_TEST_CASE("CmdLineParameters", "SubmitTool::Core::CommandLine::Parameters::Contains", "[SubmitToolCore][CommandLine][Parameters]")
{
	FCommandLine::Set(TEXT("-server perforce:9999 -client mycomputer -user testuser -cl 1234567"));
	CHECK(FCmdLineParameters::Get().Contains(TEXT("server")));
	CHECK(FCmdLineParameters::Get().Contains(TEXT("client")));
	CHECK(FCmdLineParameters::Get().Contains(TEXT("user")));
	CHECK(FCmdLineParameters::Get().Contains(TEXT("cl")));
	CHECK_FALSE(FCmdLineParameters::Get().Contains(TEXT("1234567")));
}

GROUP_TEST_CASE("CmdLineParameters", "SubmitTool::Core::CommandLine::Parameters::GetValue", "[SubmitToolCore][CommandLine][Parameters]")
{
	FString Value;

	FCommandLine::Set(TEXT("-server perforce:9999 -client mycomputer -user testuser -cl 1234567"));

	FCmdLineParameters::Get().GetValue(TEXT("server"), Value);
	CHECK_EQUALS(TEXT("Server Arg"), Value, TEXT("perforce:9999"));

	FCmdLineParameters::Get().GetValue(TEXT("client"), Value);
	CHECK_EQUALS(TEXT("Client Arg"), Value, TEXT("mycomputer"));

	FCmdLineParameters::Get().GetValue(TEXT("user"), Value);
	CHECK_EQUALS(TEXT("User Arg"), Value, TEXT("testuser"));

	FCmdLineParameters::Get().GetValue(TEXT("cl"), Value);
	CHECK_EQUALS(TEXT("CL Arg"), Value, TEXT("1234567"));
}

#endif